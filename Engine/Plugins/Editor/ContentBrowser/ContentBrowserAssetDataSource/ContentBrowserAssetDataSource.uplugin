{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Content Browser - Asset Data Source",
	"Description" : "Data Source plugin providing Asset Data to the Content Browser",
	"Category" : "Content Browser",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed": false,
	"SupportedPrograms":  ["LiveLinkHub"],
	"Modules" :
	[
		{
			"Name": "ContentBrowserAssetDataSource",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"ProgramAllowlist": [ "LiveLinkHub" ]
		}
	]
}