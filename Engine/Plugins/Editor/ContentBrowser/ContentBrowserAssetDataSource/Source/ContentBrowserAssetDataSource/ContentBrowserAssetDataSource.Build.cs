// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ContentBrowserAssetDataSource : ModuleRules
	{
		public ContentBrowserAssetDataSource(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "CBAssetDataSource";

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"ContentBrowserData",
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"ApplicationCore",
					"AssetRegistry",
					"AssetTools",
					"CollectionManager",
					"ContentBrowser",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"EditorWidgets",
					"Projects",
					"ToolMenus",
					"ToolWidgets",
					"Slate",
					"SlateCore",
					"InputCore",
					"Json",
					"SourceControl",
					"SourceControlWindows",
				}
			);

			// This module needs AutoRTFM so disable the auto disable since its in a plugin.
			// The call to the delegate `OnObjectPropertyChanged` calls into here.
			bDisableAutoRTFMInstrumentation = false;
		}
	}
}
