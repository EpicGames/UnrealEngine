// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetFileContextMenu.h"

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "Engine/Blueprint.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Engine/UserDefinedEnum.h"
#include "IAssetTypeActions.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UObject/MetaData.h"
#include "ToolMenu.h"
#include "UObject/PackageFileSummary.h"
#include "ToolMenuSection.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "ContentBrowserDataMenuContexts.h"
#include "Widgets/Input/SButton.h"
#include "EditorReimportHandler.h"
#include "UnrealClient.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/Material.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Materials/MaterialInstanceConstant.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Dialogs/Dialogs.h"
#include "SMetaDataView.h"

#include "ObjectTools.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"

#include "PropertyEditorModule.h"
#include "ConsolidateWindow.h"
#include "ReferencedAssetsUtils.h"
#include "Internationalization/PackageLocalizationUtil.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/PropertyLocalizationDataGathering.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "ComponentAssetBroker.h"
#include "ContentBrowserAssetDataSourceCommands.h"
#include "Widgets/Input/SNumericEntryBox.h"

#include "SourceCodeNavigation.h"
#include "IDocumentation.h"
#include "EditorClassUtils.h"

#include "Internationalization/Culture.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/LevelStreaming.h"

#include "PackageHelperFunctions.h"
#include "EngineUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "Commandlets/TextAssetCommandlet.h"
#include "FileHelpers.h"
#include "Misc/FileHelper.h"
#include "Internationalization/GatherableTextData.h"

#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"
#include "Algo/AnyOf.h"
#include "Misc/WarnIfAssetsLoadedInScope.h"
#include "CoreGlobalsInternal.h"
#include "Serialization/PackageStore.h"


#define LOCTEXT_NAMESPACE "ContentBrowser"

void FAssetFileContextMenu::MakeContextMenu(UToolMenu* InMenu, const TArray<FAssetData>& InSelectedAssets, const FOnShowAssetsInPathsView& InOnShowAssetsInPathsView)
{
	SelectedAssets = InSelectedAssets;
	OnShowAssetsInPathsView = InOnShowAssetsInPathsView;

	// We don't want the regular menu options if the selection contains unsupported assets
	const UContentBrowserDataMenuContext_FileMenu* Context = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	if(Context && Context->bContainsUnsupportedAssets)
	{
		return;
	}

	if (Context)
	{
		OnRefreshView = Context->OnRefreshView;
	}
	
	if (SelectedAssets.Num() > 0)
	{
		AddMenuOptions(InMenu);
	}
}

void FAssetFileContextMenu::AddMenuOptions(UToolMenu* Menu)
{
	const UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	checkf(Context, TEXT("Required context UContentBrowserDataMenuContext_FileMenu was missing!"));

	ParentWidget = Context->ParentWidget;

	// Cache any vars that are used in determining if you can execute any actions.
	// Useful for actions whose "CanExecute" will not change or is expensive to calculate.
	CacheCanExecuteVars();

	// Add imported asset context menu options
	if (Context->bCanBeModified)
	{
		AddImportedAssetMenuOptions(Menu);
	}

	// Add quick access to common commands.
	AddCommonMenuOptions(Menu);

	// Add documentation options
	AddDocumentationMenuOptions(Menu);
}

bool FAssetFileContextMenu::AddImportedAssetMenuOptions(UToolMenu* Menu)
{
	if (AreImportedAssetActionsVisible())
	{
		FToolMenuSection& Section = Menu->AddSection("ImportedAssetActions", LOCTEXT("ImportedAssetActionsMenuHeading", "Imported Asset"));
		Section.InsertPosition = FToolMenuInsert("CommonAssetActions", EToolMenuInsertType::Before);
		{
			auto CreateSubMenu = [this](UToolMenu* SubMenu, bool bReimportWithNewFile)
			{
				//Get the data, we cannot use the closure since the lambda will be call when the function scope will be gone
				TArray<FString> ResolvedFilePaths;
				TArray<FString> SourceFileLabels;
				int32 ValidSelectedAssetCount = 0;
				GetSelectedAssetSourceFilePaths(ResolvedFilePaths, SourceFileLabels, ValidSelectedAssetCount);
				if (SourceFileLabels.Num() > 0 )
				{
					FToolMenuSection& SubSection = SubMenu->AddSection("Section");
					for (int32 SourceFileIndex = 0; SourceFileIndex < SourceFileLabels.Num(); ++SourceFileIndex)
					{
						FText ReimportLabel = FText::Format(LOCTEXT("ReimportNoLabel", "SourceFile {0}"), SourceFileIndex);
						FText ReimportLabelTooltip;
						if (ValidSelectedAssetCount == 1)
						{
							ReimportLabelTooltip = FText::Format(LOCTEXT("ReimportNoLabelTooltip", "Reimport File: {0}"), FText::FromString(ResolvedFilePaths[SourceFileIndex]));
						}
						if (SourceFileLabels[SourceFileIndex].Len() > 0)
						{
							ReimportLabel = FText::Format(LOCTEXT("ReimportLabel", "{0}"), FText::FromString(SourceFileLabels[SourceFileIndex]));
							if (ValidSelectedAssetCount == 1)
							{
								ReimportLabelTooltip = FText::Format(LOCTEXT("ReimportLabelTooltip", "Reimport {0} File: {1}"), FText::FromString(SourceFileLabels[SourceFileIndex]), FText::FromString(ResolvedFilePaths[SourceFileIndex]));
							}
						}
						if (bReimportWithNewFile)
						{
							SubSection.AddMenuEntry(
								NAME_None,
								ReimportLabel,
								ReimportLabelTooltip,
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"),
								FUIAction(
									FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteReimportWithNewFile, SourceFileIndex),
									FCanExecuteAction()
								)
							);
						}
						else
						{
							SubSection.AddMenuEntry(
								NAME_None,
								ReimportLabel,
								ReimportLabelTooltip,
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"),
								FUIAction(
									FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteReimport, SourceFileIndex),
									FCanExecuteAction()
								)
							);
						}
						
					}
				}
			};
			
			TArray<FString> ResolvedFilePaths;
			TArray<FString> SourceFileLabels;
			int32 ValidSelectedAssetCount = 0;
			GetSelectedAssetSourceFilePaths(ResolvedFilePaths, SourceFileLabels, ValidSelectedAssetCount);

			//Reimport Menu
			if (ValidSelectedAssetCount == 1 && SourceFileLabels.Num() > 1)
			{
				Section.AddSubMenu(
					"Reimport",
					LOCTEXT("Reimport", "Reimport"),
					FText(),
					FNewToolMenuDelegate::CreateLambda(CreateSubMenu, false),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"));
				//With new file
				Section.AddSubMenu(
					"ReimportWithNewFile",
					LOCTEXT("ReimportWithNewFile", "Reimport With New File"),
					FText(),
					FNewToolMenuDelegate::CreateLambda(CreateSubMenu, true),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"));
			}
			else
			{
				Section.AddMenuEntry(
					"Reimport",
					LOCTEXT("Reimport", "Reimport"),
					LOCTEXT("ReimportTooltip", "Reimport the selected asset(s) from the source file on disk."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteReimport, (int32)INDEX_NONE),
						FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteReimportAssetActions, ResolvedFilePaths)
					)
				);
				if (ValidSelectedAssetCount == 1)
				{
					//With new file
					Section.AddMenuEntry(
						"ReimportWithNewFile",
						LOCTEXT("ReimportWithNewFile", "Reimport With New File"),
						LOCTEXT("ReimportWithNewFileTooltip", "Reimport the selected asset from a new source file on disk."),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"),
						FUIAction(
							FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteReimportWithNewFile, (int32)INDEX_NONE),
							FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteReimportAssetActions, ResolvedFilePaths)
						)
					);
				}
			}

			// Show Source In Explorer
			Section.AddMenuEntry(
				"FindSourceFile",
				LOCTEXT("FindSourceFile", "Open Source Location"),
				LOCTEXT("FindSourceFileTooltip", "Opens the folder containing the source of the selected asset(s)."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.OpenSourceLocation"),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteFindSourceInExplorer, ResolvedFilePaths),
					FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteImportedAssetActions, ResolvedFilePaths)
					)
				);

			// Open In External Editor
			Section.AddMenuEntry(
				"OpenInExternalEditor",
				LOCTEXT("OpenInExternalEditor", "Open In External Editor"),
				LOCTEXT("OpenInExternalEditorTooltip", "Open the selected asset(s) in the default external editor."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.OpenInExternalEditor"),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteOpenInExternalEditor, ResolvedFilePaths),
					FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteImportedAssetActions, ResolvedFilePaths)
					)
				);
		}

		return true;
	}

	return false;
}

bool FAssetFileContextMenu::AddCommonMenuOptions(UToolMenu* Menu)
{
	UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	const bool bCanBeModified = !Context || Context->bCanBeModified;

	{
		FToolMenuSection& Section = Menu->AddSection("CommonAssetActions", LOCTEXT("CommonAssetActionsMenuHeading", "Common"));

		// Asset Actions sub-menu
		Section.AddSubMenu(
			"AssetActionsSubMenu",
			LOCTEXT("AssetActionsSubMenuLabel", "Asset Actions"),
			LOCTEXT("AssetActionsSubMenuToolTip", "Other asset actions"),
			FNewToolMenuDelegate::CreateSP(this, &FAssetFileContextMenu::MakeAssetActionsSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateSP( this, &FAssetFileContextMenu::CanExecuteAssetActions )
				),
			EUserInterfaceActionType::Button,
			false, 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Adjust")
			);

		if (bCanBeModified)
		{
			// Asset Localization sub-menu
			Section.AddSubMenu(
				"LocalizationSubMenu",
				LOCTEXT("LocalizationSubMenuLabel", "Asset Localization"),
				LOCTEXT("LocalizationSubMenuToolTip", "Manage the localization of this asset"),
				FNewToolMenuDelegate::CreateSP(this, &FAssetFileContextMenu::MakeAssetLocalizationSubMenu),
				FUIAction(),
				EUserInterfaceActionType::Button,
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Localization")
			);
		}
	}

	return true;
}

void FAssetFileContextMenu::MakeAssetActionsSubMenu(UToolMenu* Menu)
{
	FWarnIfAssetsLoadedInScope WarnIfAssetsLoaded;
	
	UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	const bool bCanBeModified = !Context || Context->bCanBeModified;

	{
		FToolMenuSection& Section = Menu->AddSection("AssetActionsSection");

		if (bCanBeModified)
		{
			// Create BP Using This
			Section.AddMenuEntry(
				"CreateBlueprintUsing",
				LOCTEXT("CreateBlueprintUsing", "Create Blueprint Using This..."),
				LOCTEXT("CreateBlueprintUsingTooltip", "Create a new Blueprint and add this asset to it"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteCreateBlueprintUsing),
					FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteCreateBlueprintUsing)
				)
			);
		}

		// Capture Thumbnail
		if (bCanBeModified && !SelectedAssets.IsEmpty())
		{
			const FContentBrowserAssetDataSourceCommands& AssetDataSourceCommands = FContentBrowserAssetDataSourceCommands::Get();
			Section.AddMenuEntry(AssetDataSourceCommands.CaptureThumbnail);
		}

		// Clear Thumbnail
		if (bCanBeModified && CanClearCustomThumbnails())
		{
			Section.AddMenuEntry(
				"ClearCustomThumbnail",
				LOCTEXT("ClearCustomThumbnail", "Clear Thumbnail"),
				LOCTEXT("ClearCustomThumbnailTooltip", "Clears all custom thumbnails for selected assets."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteClearThumbnail))
			);
		}
	}

	// FIND ACTIONS
	{
		FToolMenuSection& Section = Menu->AddSection("AssetContextFindActions", LOCTEXT("AssetContextFindActionsMenuHeading", "Find"));
		// Select Actors Using This Asset
		Section.AddMenuEntry(
			"FindAssetInWorld",
			LOCTEXT("FindAssetInWorld", "Select Actors Using This Asset"),
			LOCTEXT("FindAssetInWorldTooltip", "Selects all actors referencing this asset."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteFindAssetInWorld),
				FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteFindAssetInWorld)
				)
			);
	}

	// MOVE ACTIONS
	if (bCanBeModified)
	{
		FToolMenuSection& Section = Menu->AddSection("AssetContextMoveActions", LOCTEXT("AssetContextMoveActionsMenuHeading", "Move"));
		const bool bCanExportAllSelectedAssets = FAssetToolsModule::GetModule().Get().CanExportAssets(SelectedAssets);

		if (bCanExportAllSelectedAssets)
		{
			// Export
			Section.AddMenuEntry(
				"Export",
				LOCTEXT("Export", "Export..."),
				LOCTEXT("ExportTooltip", "Export the selected assets to file."),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteExport ) )
				);

			// Bulk Export
			if (SelectedAssets.Num() > 1)
			{
				Section.AddMenuEntry(
					"BulkExport",
					LOCTEXT("BulkExport", "Bulk Export..."),
					LOCTEXT("BulkExportTooltip", "Export the selected assets to file in the selected directory"),
					FSlateIcon(),
					FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteBulkExport ) )
					);
			}
		}

		// Migrate
		Section.AddMenuEntry(
			"MigrateAsset",
			LOCTEXT("MigrateAsset", "Migrate..."),
			LOCTEXT("MigrateAssetTooltip", "Copies all selected assets and their dependencies to another project"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.Migrate"),
			FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteMigrateAsset ) )
			);
	}

	// ADVANCED ACTIONS
	{
		FToolMenuSection& Section = Menu->AddSection("AssetContextAdvancedActions", LOCTEXT("AssetContextAdvancedActionsMenuHeading", "Advanced"));

		// Reload
		Section.AddMenuEntry(
			"Reload",
			LOCTEXT("Reload", "Reload"),
			LOCTEXT("ReloadTooltip", "Reload the selected assets from their file on disk."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteReload),
				FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteReload)
			)
		);

		if (IsRunningHybridCookedEditor())
		{
			// Reload
			Section.AddMenuEntry(
				"ReloadUncooked",
				LOCTEXT("ReloadUncooked", "Reload Uncooked Version"),
				LOCTEXT("ReloadUncookedTooltip", "Reload the selected assets from their uncooked file on disk."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteReloadUncooked),
					FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteReloadUncooked)
				)
			);
		}


		// Load
		Section.AddMenuEntry(
			"Load",
			LOCTEXT("Load", "Load"),
			LOCTEXT("LoadTooltip", "Loads the selected assets into memory without opening any asset editors.\nShift+Click to fully load the selected assets so that UE no longer has a write lock on their on-disk files."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteLoad),
				FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecuteLoad)
			)
		);

		// Replace References
		if (bCanBeModified)
		{
			Section.AddMenuEntry(
				"ReplaceReferences",
				LOCTEXT("ReplaceReferences", "Replace References"),
				LOCTEXT("ConsolidateTooltip", "Replace references to the selected assets."),
				FSlateIcon(),
				FUIAction(
				FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteConsolidate)
				)
			);
		}

		// Property Matrix
		bool bCanUsePropertyMatrix = bCanBeModified && FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" ).GetCanUsePropertyMatrix();
		if (bCanUsePropertyMatrix)
		{
			// Materials can't be bulk edited currently as they require very special handling because of their dependencies with the rendering thread, and we'd have to hack the property matrix too much.
			for (const FAssetData& Asset : SelectedAssets)
			{
				if (Asset.AssetClassPath == UMaterialInstanceConstant::StaticClass()->GetClassPathName() || Asset.AssetClassPath == UMaterialFunction::StaticClass()->GetClassPathName() || Asset.AssetClassPath == UMaterialFunctionInstance::StaticClass()->GetClassPathName())
				{
					bCanUsePropertyMatrix = false;
					break;
				}
			}
		}

		if (bCanUsePropertyMatrix)
		{
			TAttribute<FText>::FGetter DynamicTooltipGetter;
			DynamicTooltipGetter.BindSP(this, &FAssetFileContextMenu::GetExecutePropertyMatrixTooltip);
			TAttribute<FText> DynamicTooltipAttribute = TAttribute<FText>::Create(DynamicTooltipGetter);

			Section.AddMenuEntry(
				"PropertyMatrix",
				LOCTEXT("PropertyMatrix", "Edit Selection in Property Matrix"),
				DynamicTooltipAttribute,
				FSlateIcon(),
				FUIAction(
				FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecutePropertyMatrix),
				FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanExecutePropertyMatrix)
				)
				);
		}

		if (bCanBeModified)
		{
			// Create Metadata menu
			Section.AddMenuEntry(
				"ShowAssetMetaData",
				LOCTEXT("ShowAssetMetaData", "Show Metadata"),
				LOCTEXT("ShowAssetMetaDataTooltip", "Show the asset metadata dialog."),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteShowAssetMetaData) )
			);
		}

		if (bCanBeModified)
		{
			Section.AddMenuEntry(
				"CopyAssetRegistryTags",
				LOCTEXT("CopyAssetRegistryTags", "Copy Asset Registry Tags"),
				LOCTEXT("CopyAssetRegistryTagsTooltip", "Copies to clipboard all asset registry tags that belong to this asset."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteCopyAssetRegistryTags))
			);
		}

		// Chunk actions
		if (bCanBeModified && GetDefault<UEditorExperimentalSettings>()->bContextMenuChunkAssignments)
		{
			Section.AddMenuEntry(
				"AssignAssetChunk",
				LOCTEXT("AssignAssetChunk", "Assign to Chunk..."),
				LOCTEXT("AssignAssetChunkTooltip", "Assign this asset to a specific Chunk"),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteAssignChunkID) )
				);

			Section.AddSubMenu(
				"RemoveAssetFromChunk",
				LOCTEXT("RemoveAssetFromChunk", "Remove from Chunk..."),
				LOCTEXT("RemoveAssetFromChunkTooltip", "Removed an asset from a Chunk it's assigned to."),
				FNewToolMenuDelegate::CreateRaw(this, &FAssetFileContextMenu::MakeChunkIDListMenu)
				);

			Section.AddMenuEntry(
				"RemoveAllChunkAssignments",
				LOCTEXT("RemoveAllChunkAssignments", "Remove from all Chunks"),
				LOCTEXT("RemoveAllChunkAssignmentsTooltip", "Removed an asset from all Chunks it's assigned to."),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteRemoveAllChunkID) )
				);
		}
	}

	if (bCanBeModified && GetDefault<UEditorExperimentalSettings>()->bTextAssetFormatSupport)
	{
		FToolMenuSection& FormatActionsSection = Menu->AddSection("AssetContextTextAssetFormatActions", LOCTEXT("AssetContextTextAssetFormatActionsHeading", "Text Assets"));
		{
			FormatActionsSection.AddMenuEntry(
				"ExportToTextFormat",
				LOCTEXT("ExportToTextFormat", "Export to text format"),
				LOCTEXT("ExportToTextFormatTooltip", "Exports the selected asset(s) to the experimental text asset format"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExportSelectedAssetsToText))
			);

			FormatActionsSection.AddMenuEntry(
				"ViewSelectedAssetAsText",
				LOCTEXT("ViewSelectedAssetAsText", "View as text"),
				LOCTEXT("ViewSelectedAssetAsTextTooltip", "Opens a window showing the selected asset in text format"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ViewSelectedAssetAsText),
					FCanExecuteAction::CreateSP(this, &FAssetFileContextMenu::CanViewSelectedAssetAsText))
			);

			FormatActionsSection.AddMenuEntry(
				"RunTextAssetRoundtrip",
				LOCTEXT("TextFormatRountrip", "Run Text Asset Roundtrip"),
				LOCTEXT("TextFormatRountripTooltip", "Save the select asset backwards or forwards between text and binary formats and check for determinism"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::DoTextFormatRoundtrip))
			);
		}
	}
}

void FAssetFileContextMenu::ExportSelectedAssetsToText()
{
	FString FailedPackage;
	for (const FAssetData& Asset : SelectedAssets)
	{
		UPackage* Package = Asset.GetPackage();
		FString Filename = FPackageName::LongPackageNameToFilename(Package->GetPathName(), FPackageName::GetTextAssetPackageExtension());
		if (!SavePackageHelper(Package, Filename))
		{
			FailedPackage = Package->GetPathName();
			break;
		}
	}

	if (FailedPackage.Len() > 0)
	{
		FNotificationInfo Info(LOCTEXT("ExportedTextAssetFailed", "Exported selected asset(s) failed"));
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
	else
	{
		FNotificationInfo Info(LOCTEXT("ExportedTextAssetsSuccessfully", "Exported selected asset(s) successfully"));
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FAssetFileContextMenu::ViewSelectedAssetAsText()
{
	if (SelectedAssets.Num() == 1)
	{
		UPackage* Package = SelectedAssets[0].GetPackage();
		FString TargetFilename = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), nullptr, *FPackageName::GetTextAssetPackageExtension());
		if (SavePackageHelper(Package, TargetFilename))
		{
			FString TextFormat;
			if (FFileHelper::LoadFileToString(TextFormat, *TargetFilename))
			{
				SGenericDialogWidget::OpenDialog(LOCTEXT("TextAssetViewerTitle", "Viewing AS Text Asset..."), SNew(STextBlock).Text(FText::FromString(TextFormat)));
			}
			IFileManager::Get().Delete(*TargetFilename);
		}
	}
}

bool FAssetFileContextMenu::CanViewSelectedAssetAsText() const
{
	return SelectedAssets.Num() == 1;
}

void FAssetFileContextMenu::DoTextFormatRoundtrip()
{
	UTextAssetCommandlet::FProcessingArgs Args;
	Args.NumSaveIterations = 1;
	Args.bIncludeEngineContent = true;
	Args.bVerifyJson = true;
	Args.CSVFilename = TEXT("");
	Args.ProcessingMode = ETextAssetCommandletMode::RoundTrip;
	Args.bFilenameIsFilter = false;

	for (const FAssetData& Asset : SelectedAssets)
	{
		UPackage* Package = Asset.GetPackage();
		Args.Filename = FPackageName::LongPackageNameToFilename(Package->GetPathName());
		if (!UTextAssetCommandlet::DoTextAssetProcessing(Args))
		{
			FNotificationInfo Info(LOCTEXT("RountripTextAssetFailed", "Roundtripping of selected asset(s) failed"));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return;
		}
	}

	FNotificationInfo Info(LOCTEXT("RoundtripTextAssetsSuccessfully", "Roundtripped selected asset(s) successfully"));
	Info.ExpireDuration = 3.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
}

bool FAssetFileContextMenu::CanExecuteAssetActions() const
{
	return SelectedAssets.Num() > 0;
}

void FAssetFileContextMenu::MakeAssetLocalizationSubMenu(UToolMenu* Menu)
{
	FWarnIfAssetsLoadedInScope WarnIfAssetsLoaded;
	
	TArray<FCultureRef> CurrentCultures;

	// Build up the list of cultures already used
	{
		TSet<FString> CultureNames;

		bool bIncludeEngineCultures = false;
		bool bIncludeProjectCultures = false;

		TSet<FString> Roots;
		for (const FAssetData& Asset : SelectedAssets)
		{
			const FString AssetPath = Asset.GetObjectPathString();

			if (AssetViewUtils::IsEngineFolder(AssetPath))
			{
				bIncludeEngineCultures = true;
			}
			else
			{
				bIncludeProjectCultures = true;
			}

			FString AssetLocalizationRoot;
			if (FPackageLocalizationUtil::GetLocalizedRoot(AssetPath, FString(), AssetLocalizationRoot))
			{
				Roots.Add(AssetLocalizationRoot);	
			}
		}

		for (const FString& AssetLocalizationRoot : Roots)
		{
			FString AssetLocalizationFileRoot;
			if (FPackageName::TryConvertLongPackageNameToFilename(AssetLocalizationRoot, AssetLocalizationFileRoot))
			{
				TArray<FString> CulturePaths;
				CulturePaths.Add(MoveTemp(AssetLocalizationFileRoot));
				CultureNames.Append(TextLocalizationResourceUtil::GetLocalizedCultureNames(CulturePaths));
			}
		}

		ELocalizationLoadFlags LocLoadFlags = ELocalizationLoadFlags::None;
		if (bIncludeEngineCultures)
		{
			LocLoadFlags |= ELocalizationLoadFlags::Engine;
		}
		if (bIncludeProjectCultures)
		{
			LocLoadFlags |= ELocalizationLoadFlags::Game;
		}
		CultureNames.Append(FTextLocalizationManager::Get().GetLocalizedCultureNames(LocLoadFlags));

		CurrentCultures = FInternationalization::Get().GetAvailableCultures(CultureNames.Array(), false);
		if (CurrentCultures.Num() == 0)
		{
			CurrentCultures.Add(FInternationalization::Get().GetCurrentCulture());
		}
	}

	// Sort by display name for the UI
	CurrentCultures.Sort([](const FCultureRef& FirstCulture, const FCultureRef& SecondCulture) -> bool
	{
		const FText FirstDisplayName = FText::FromString(FirstCulture->GetDisplayName());
		const FText SecondDisplayName = FText::FromString(SecondCulture->GetDisplayName());
		return FirstDisplayName.CompareTo(SecondDisplayName) < 0;
	});

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	// Now build up the list of available localized or source assets based upon the current selection and current cultures
	FSourceAssetsState SourceAssetsState;
	TArray<FLocalizedAssetsState> LocalizedAssetsState;
	for (const FCultureRef& CurrentCulture : CurrentCultures)
	{
		FLocalizedAssetsState& LocalizedAssetsStateForCulture = LocalizedAssetsState[LocalizedAssetsState.AddDefaulted()];
		LocalizedAssetsStateForCulture.Culture = CurrentCulture;

		for (const FAssetData& Asset : SelectedAssets)
		{
			// Can this type of asset be localized?
			const bool bCanLocalizeAsset = AssetToolsModule.Get().CanLocalize(Asset.GetClass());

			if (!bCanLocalizeAsset)
			{
				continue;
			}

			const FString ObjectPath = Asset.GetObjectPathString();
			if (FPackageName::IsLocalizedPackage(ObjectPath))
			{
				// Get the source path for this asset
				FString SourceObjectPath;
				if (FPackageLocalizationUtil::ConvertLocalizedToSource(ObjectPath, SourceObjectPath))
				{
					SourceAssetsState.CurrentAssets.Add(FSoftObjectPath(SourceObjectPath));
				}
			}
			else
			{
				SourceAssetsState.SelectedAssets.Add(Asset.GetSoftObjectPath());

				// Get the localized path for this asset and culture
				FString LocalizedObjectPath;
				if (FPackageLocalizationUtil::ConvertSourceToLocalized(ObjectPath, CurrentCulture->GetName(), LocalizedObjectPath))
				{
					// Does this localized asset already exist?
					// Need to check for both UE and Verse style identifiers, as both may be in use
					FAssetData LocalizedAssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath(LocalizedObjectPath));
					if (!LocalizedAssetData.IsValid())
					{
						if (const FString VerseIdentifier = CurrentCulture->GetVerseIdentifier();
							VerseIdentifier != CurrentCulture->GetName())
						{
							verify(FPackageLocalizationUtil::ConvertSourceToLocalized(ObjectPath, VerseIdentifier, LocalizedObjectPath));
							LocalizedAssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath(LocalizedObjectPath));
						}
					}

					if (LocalizedAssetData.IsValid())
					{
						LocalizedAssetsStateForCulture.CurrentAssets.Add(FSoftObjectPath(LocalizedObjectPath));
					}
				}
			}
		}
	}

#if USE_STABLE_LOCALIZATION_KEYS
	// Add the Localization ID options
	{
		FToolMenuSection& Section = Menu->AddSection("LocalizationId", LOCTEXT("LocalizationIdHeading", "Localization ID"));
		{
			// Show the localization ID if we have a single asset selected
			if (SelectedAssets.Num() == 1)
			{
				FAssetData LocalizationAsset = SelectedAssets[0]; 
				Section.AddMenuEntry(
					"CopyLocalizationId",
					LOCTEXT("CopyLocalizationId", "Copy Localization ID"),
					LOCTEXT("CopyLocalizationIdTooltip", "Load the asset and copy the localization ID to the clipboard."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([LocalizationAsset]()
					{
						const FString LocalizationId = TextNamespaceUtil::GetPackageNamespace(LocalizationAsset.FastGetAsset(true));
						FPlatformApplicationMisc::ClipboardCopy(*LocalizationId);
					}))
				);
			}

			// Always show the reset localization ID option
			Section.AddMenuEntry(
				"ResetLocalizationId",
				LOCTEXT("ResetLocalizationId", "Reset Localization ID"),
				LOCTEXT("ResetLocalizationIdTooltip", "Reset the localization ID. Note: This will re-key all the text within this asset."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteResetLocalizationId))
				);
		}
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

	// Add the localization cache options
	if (SelectedAssets.Num() == 1)
	{
		FString PackageFilename;
		if (FPackageName::DoesPackageExist(SelectedAssets[0].PackageName.ToString(), &PackageFilename))
		{
			FToolMenuSection& Section = Menu->AddSection("LocalizationCache", LOCTEXT("LocalizationCacheHeading", "Localization Cache"));
			{
				// Always show the reset localization ID option
				Section.AddMenuEntry(
					"ShowLocalizationCache",
					LOCTEXT("ShowLocalizationCache", "Show Localization Cache"),
					LOCTEXT("ShowLocalizationCacheTooltip", "Show the cached list of localized texts stored in the package header."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteShowLocalizationCache, SelectedAssets[0], PackageFilename))
					);
			}
		}
	}

	// If we found source assets for localized assets, then we can show the Source Asset options
	if (SourceAssetsState.CurrentAssets.Num() > 0)
	{
		FToolMenuSection& Section = Menu->AddSection("ManageSourceAsset", LOCTEXT("ManageSourceAssetHeading", "Manage Source Asset"));
		{
			Section.AddMenuEntry(
				"ShowSourceAsset",
				LOCTEXT("ShowSourceAsset", "Show Source Asset"),
				LOCTEXT("ShowSourceAssetTooltip", "Show the source asset in the Content Browser."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser"),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteFindInAssetTree, SourceAssetsState.CurrentAssets.Array()))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				);

			Section.AddMenuEntry(
				"EditSourceAsset",
				LOCTEXT("EditSourceAsset", "Edit Source Asset"),
				LOCTEXT("EditSourceAssetTooltip", "Edit the source asset."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Edit"),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteOpenEditorsForAssets, SourceAssetsState.CurrentAssets.Array()))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				);
		}
	}

	// If we currently have source assets selected, then we can show the Localized Asset options
	if (SourceAssetsState.SelectedAssets.Num() > 0)
	{
		FToolMenuSection& Section = Menu->AddSection("ManageLocalizedAsset", LOCTEXT("ManageLocalizedAssetHeading", "Manage Localized Asset"));
		{
			Section.AddSubMenu(
				"CreateLocalizedAsset",
				LOCTEXT("CreateLocalizedAsset", "Create Localized Asset"),
				LOCTEXT("CreateLocalizedAssetTooltip", "Create a new localized asset."),
				FNewToolMenuDelegate::CreateSP(this, &FAssetFileContextMenu::MakeCreateLocalizedAssetSubMenu, SourceAssetsState.SelectedAssets, LocalizedAssetsState),
				FUIAction(),
				EUserInterfaceActionType::Button
				);

			int32 NumLocalizedAssets = 0;
			for (const FLocalizedAssetsState& LocalizedAssetsStateForCulture : LocalizedAssetsState)
			{
				NumLocalizedAssets += LocalizedAssetsStateForCulture.CurrentAssets.Num();
			}

			if (NumLocalizedAssets > 0)
			{
				Section.AddSubMenu(
					"ShowLocalizedAsset",
					LOCTEXT("ShowLocalizedAsset", "Show Localized Asset"),
					LOCTEXT("ShowLocalizedAssetTooltip", "Show the localized asset in the Content Browser."),
					FNewToolMenuDelegate::CreateSP(this, &FAssetFileContextMenu::MakeShowLocalizedAssetSubMenu, LocalizedAssetsState),
					FUIAction(),
					EUserInterfaceActionType::Button
					);

				Section.AddSubMenu(
					"EditLocalizedAsset",
					LOCTEXT("EditLocalizedAsset", "Edit Localized Asset"),
					LOCTEXT("EditLocalizedAssetTooltip", "Edit the localized asset."),
					FNewToolMenuDelegate::CreateSP(this, &FAssetFileContextMenu::MakeEditLocalizedAssetSubMenu, LocalizedAssetsState),
					FUIAction(),
					EUserInterfaceActionType::Button
					);
			}
		}
	}
}

void FAssetFileContextMenu::MakeCreateLocalizedAssetSubMenu(UToolMenu* Menu, TSet<FSoftObjectPath> InSelectedSourceAssets, TArray<FLocalizedAssetsState> InLocalizedAssetsState)
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	for (const FLocalizedAssetsState& LocalizedAssetsStateForCulture : InLocalizedAssetsState)
	{
		// If we have less localized assets than we have selected source assets, then we'll have some assets to create localized variants of
		if (LocalizedAssetsStateForCulture.CurrentAssets.Num() < InSelectedSourceAssets.Num())
		{
			Section.AddMenuEntry(
				NAME_None,
				FText::FromString(LocalizedAssetsStateForCulture.Culture->GetDisplayName()),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteCreateLocalizedAsset, InSelectedSourceAssets, LocalizedAssetsStateForCulture))
				);
		}
	}
}

void FAssetFileContextMenu::MakeShowLocalizedAssetSubMenu(UToolMenu* Menu, TArray<FLocalizedAssetsState> InLocalizedAssetsState)
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	for (const FLocalizedAssetsState& LocalizedAssetsStateForCulture : InLocalizedAssetsState)
	{
		if (LocalizedAssetsStateForCulture.CurrentAssets.Num() > 0)
		{
			Section.AddMenuEntry(
				NAME_None,
				FText::FromString(LocalizedAssetsStateForCulture.Culture->GetDisplayName()),
				FText::GetEmpty(),
				FSlateIcon(),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteFindInAssetTree, LocalizedAssetsStateForCulture.CurrentAssets.Array()))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				);
		}
	}
}

void FAssetFileContextMenu::MakeEditLocalizedAssetSubMenu(UToolMenu* Menu, TArray<FLocalizedAssetsState> InLocalizedAssetsState)
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	for (const FLocalizedAssetsState& LocalizedAssetsStateForCulture : InLocalizedAssetsState)
	{
		if (LocalizedAssetsStateForCulture.CurrentAssets.Num() > 0)
		{
			Section.AddMenuEntry(
				NAME_None,
				FText::FromString(LocalizedAssetsStateForCulture.Culture->GetDisplayName()),
				FText::GetEmpty(),
				FSlateIcon(),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FUIAction(FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteOpenEditorsForAssets, LocalizedAssetsStateForCulture.CurrentAssets.Array()))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				);
		}
	}
}

void FAssetFileContextMenu::ExecuteCreateLocalizedAsset(TSet<FSoftObjectPath> InSelectedSourceAssets, FLocalizedAssetsState InLocalizedAssetsStateForCulture)
{
	TArray<UPackage*> PackagesToSave;
	TArray<FAssetData> NewObjects;

	for (const FSoftObjectPath& SourceAsset: InSelectedSourceAssets)
	{
		if (InLocalizedAssetsStateForCulture.CurrentAssets.Contains(SourceAsset))
		{
			// Asset is already localized
			continue;
		}

		UObject* SourceAssetObject = LoadObject<UObject>(nullptr, *SourceAsset.ToString());
		if (!SourceAssetObject)
		{
			// Source object cannot be loaded
			continue;
		}

		// Always create new assets using the preferred on-disk culture identifier for this project
		FString LocalizedPackageName;
		if (!FPackageLocalizationUtil::ConvertSourceToLocalized(SourceAssetObject->GetOutermost()->GetPathName(), InLocalizedAssetsStateForCulture.Culture->GetPreferredOnDiskIdentifier(), LocalizedPackageName))
		{
			continue;
		}

		ObjectTools::FPackageGroupName NewAssetName;
		NewAssetName.PackageName = LocalizedPackageName;
		NewAssetName.ObjectName = SourceAssetObject->GetName();

		TSet<UPackage*> PackagesNotDuplicated;
		UObject* NewObject = ObjectTools::DuplicateSingleObject(SourceAssetObject, NewAssetName, PackagesNotDuplicated);
		if (NewObject)
		{
			PackagesToSave.Add(NewObject->GetOutermost());
			NewObjects.Add(FAssetData(NewObject));
		}
	}

	if (PackagesToSave.Num() > 0)
	{
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty*/false, /*bPromptToSave*/false);
	}

	OnShowAssetsInPathsView.ExecuteIfBound(NewObjects);
}

void FAssetFileContextMenu::ExecuteFindInAssetTree(TArray<FSoftObjectPath> InAssets)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	FARFilter ARFilter;
	ARFilter.SoftObjectPaths = InAssets;
	
	TArray<FAssetData> FoundLocalizedAssetData;
	AssetRegistryModule.Get().GetAssets(ARFilter, FoundLocalizedAssetData);

	OnShowAssetsInPathsView.ExecuteIfBound(FoundLocalizedAssetData);
}

void FAssetFileContextMenu::ExecuteOpenEditorsForAssets(TArray<FSoftObjectPath> InAssets)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(InAssets);
}

bool FAssetFileContextMenu::AddDocumentationMenuOptions(UToolMenu* Menu)
{
	bool bAddedOption = false;

	// Objects must be loaded for this operation... for now
	UClass* SelectedClass = (SelectedAssets.Num() > 0 ? SelectedAssets[0].GetClass() : nullptr);
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (SelectedClass != AssetData.GetClass())
		{
			SelectedClass = nullptr;
			break;
		}
	}

	// Go to C++ Code
	if( SelectedClass != nullptr )
	{
		// Blueprints are special.  We won't link to C++ and for documentation we'll use the class it is generated from
		const bool bIsBlueprint = SelectedClass->IsChildOf<UBlueprint>();
		if (bIsBlueprint)
		{
			const FString ParentClassPath = SelectedAssets[0].GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint,ParentClass));
			if (!ParentClassPath.IsEmpty())
			{
				SelectedClass = FindObject<UClass>(nullptr,*ParentClassPath);
			}
		}

		if (!bIsBlueprint && FSourceCodeNavigation::IsCompilerAvailable() && ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
		{
			FString ClassHeaderPath;
			if( FSourceCodeNavigation::FindClassHeaderPath( SelectedClass, ClassHeaderPath ) && IFileManager::Get().FileSize( *ClassHeaderPath ) != INDEX_NONE )
			{
				bAddedOption = true;

				const FString CodeFileName = FPaths::GetCleanFilename( *ClassHeaderPath );

				FToolMenuSection& Section = Menu->AddSection( "AssetCode"/*, LOCTEXT("AssetCodeHeading", "C++")*/ );
				{
					Section.AddMenuEntry(
						"GoToCodeForAsset",
						FText::Format( LOCTEXT("GoToCodeForAsset", "Open {0}"), FText::FromString( CodeFileName ) ),
						FText::Format( LOCTEXT("GoToCodeForAsset_ToolTip", "Opens the header file for this asset ({0}) in a code editing program"), FText::FromString( CodeFileName ) ),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.C++"),
						FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToCodeForAsset, SelectedClass ) )
						);
				}
			}
		}

		const FString DocumentationLink = FEditorClassUtils::GetDocumentationLink(SelectedClass);
		if (bIsBlueprint || !DocumentationLink.IsEmpty())
		{
			bAddedOption = true;

			FToolMenuSection& Section = Menu->AddSection( "AssetDocumentation"/*, LOCTEXT("AseetDocsHeading", "Documentation")*/ );
			{
					if (bIsBlueprint)
					{
						if (!DocumentationLink.IsEmpty())
						{
							Section.AddMenuEntry(
								"GoToDocsForAssetWithClass",
								FText::Format( LOCTEXT("GoToDocsForAssetWithClass", "View Documentation - {0}"), SelectedClass->GetDisplayNameText() ),
								FText::Format( LOCTEXT("GoToDocsForAssetWithClass_ToolTip", "Click to open documentation for {0}"), SelectedClass->GetDisplayNameText() ),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation" ),
								FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToDocsForAsset, SelectedClass ) )
								);
						}

						UEnum* BlueprintTypeEnum = StaticEnum<EBlueprintType>();
						const FString EnumString = SelectedAssets[0].GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint,BlueprintType));
						EBlueprintType BlueprintType = (!EnumString.IsEmpty() ? (EBlueprintType)BlueprintTypeEnum->GetValueByName(*EnumString) : BPTYPE_Normal);

						switch (BlueprintType)
						{
						case BPTYPE_FunctionLibrary:
							Section.AddMenuEntry(
								"GoToDocsForMacroBlueprint",
								LOCTEXT("GoToDocsForMacroBlueprint", "View Documentation - Function Library"),
								LOCTEXT("GoToDocsForMacroBlueprint_ToolTip", "Click to open documentation on blueprint function libraries"),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation" ),
								FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToDocsForAsset, UBlueprint::StaticClass(), FString(TEXT("UBlueprint_FunctionLibrary")) ) )
								);
							break;
						case BPTYPE_Interface:
							Section.AddMenuEntry(
								"GoToDocsForInterfaceBlueprint",
								LOCTEXT("GoToDocsForInterfaceBlueprint", "View Documentation - Interface"),
								LOCTEXT("GoToDocsForInterfaceBlueprint_ToolTip", "Click to open documentation on blueprint interfaces"),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation" ),
								FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToDocsForAsset, UBlueprint::StaticClass(), FString(TEXT("UBlueprint_Interface")) ) )
								);
							break;
						case BPTYPE_MacroLibrary:
							Section.AddMenuEntry(
								"GoToDocsForMacroLibrary",
								LOCTEXT("GoToDocsForMacroLibrary", "View Documentation - Macro"),
								LOCTEXT("GoToDocsForMacroLibrary_ToolTip", "Click to open documentation on blueprint macros"),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation" ),
								FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToDocsForAsset, UBlueprint::StaticClass(), FString(TEXT("UBlueprint_Macro")) ) )
								);
							break;
						default:
							Section.AddMenuEntry(
								"GoToDocsForBlueprint",
								LOCTEXT("GoToDocsForBlueprint", "View Documentation - Blueprint"),
								LOCTEXT("GoToDocsForBlueprint_ToolTip", "Click to open documentation on blueprints"),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation" ),
								FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToDocsForAsset, UBlueprint::StaticClass(), FString(TEXT("UBlueprint")) ) )
								);
						}
					}
					else
					{
						Section.AddMenuEntry(
							"GoToDocsForAsset",
							LOCTEXT("GoToDocsForAsset", "View Documentation"),
							LOCTEXT("GoToDocsForAsset_ToolTip", "Click to open documentation"),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation" ),
							FUIAction( FExecuteAction::CreateSP( this, &FAssetFileContextMenu::ExecuteGoToDocsForAsset, SelectedClass ) )
							);
					}
			}
		}
	}

	return bAddedOption;
}

bool FAssetFileContextMenu::AreImportedAssetActionsVisible() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	
	// Check that all of the selected assets are imported
	for (auto& SelectedAsset : SelectedAssets)
	{
		auto AssetClass = SelectedAsset.GetClass();
		if (AssetClass)
		{
			auto AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetClass).Pin();
			if (!AssetTypeActions.IsValid() || !AssetTypeActions->IsImportedAsset())
			{
				return false;
			}
		}
	}

	return true;
}

bool FAssetFileContextMenu::CanExecuteImportedAssetActions(const TArray<FString> ResolvedFilePaths) const
{
	if (ResolvedFilePaths.Num() == 0)
	{
		return false;
	}

	// Verify that all the file paths are legitimate
	for (const auto& SourceFilePath : ResolvedFilePaths)
	{
		if (!SourceFilePath.Len() || IFileManager::Get().FileSize(*SourceFilePath) == INDEX_NONE)
		{
			return false;
		}
	}

	return true;
}

bool FAssetFileContextMenu::CanExecuteReimportAssetActions(const TArray<FString> ResolvedFilePaths) const
{
	if (ResolvedFilePaths.Num() == 0)
	{
		return false;
	}

	// Verify that all the file paths are non-empty
	for (const auto& SourceFilePath : ResolvedFilePaths)
	{
		if (!SourceFilePath.Len())
		{
			return false;
		}
	}

	return true;
}

void FAssetFileContextMenu::ExecuteReimport(int32 SourceFileIndex /*= INDEX_NONE*/)
{
	// Reimport all selected assets
	TArray<UObject *> CopyOfSelectedAssets;
	for (const FAssetData &SelectedAsset : SelectedAssets)
	{
		UObject *Asset = SelectedAsset.GetAsset();
		if (Asset)
		{
			CopyOfSelectedAssets.Add(Asset);
		}
	}
	FReimportManager::Instance()->ValidateAllSourceFileAndReimport(CopyOfSelectedAssets, true, SourceFileIndex, false);
}

void FAssetFileContextMenu::ExecuteReimportWithNewFile(int32 SourceFileIndex /*= INDEX_NONE*/)
{
	// Ask for a new files and reimport the selected asset
	check(SelectedAssets.Num() == 1);

	if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(SelectedAssets[0]))
	{
		FAssetData SelectedAsset = SelectedAssets[0];

		FAssetSourceFilesArgs GetSourceFilesArgs;
		GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&SelectedAsset, 1);

		// Doesn't need to resolve the paths so it is a bit quicker
		GetSourceFilesArgs.FilePathFormat = EPathUse::Display;

		int32 SourceFileCount = 0;
		AssetDefinition->GetSourceFiles(GetSourceFilesArgs, [&SourceFileCount, SourceFileIndex](const FAssetSourceFilesResult& AssetImportInfo)
		{
			++SourceFileCount;
			if (SourceFileIndex < SourceFileCount)
			{
				return false;
			}

			return true;
		});

		int32 SourceFileIndexToReplace = SourceFileIndex;
		//Check if the data is valid
		if (SourceFileIndex == INDEX_NONE)
		{
			//Ask for a new file for the index 0
			SourceFileIndexToReplace = 0;
		}
		else
		{
			// Validate that the index is validate
			check(SourceFileIndex >= 0 && SourceFileIndex < SourceFileCount);
		}

		TArray<UObject*> LoadedAssets = { SelectedAsset.GetAsset() };
		FReimportManager::Instance()->ValidateAllSourceFileAndReimport(LoadedAssets, true, SourceFileIndexToReplace, true);
	}
}

void FAssetFileContextMenu::ExecuteFindSourceInExplorer(const TArray<FString> ResolvedFilePaths)
{
	// Open all files in the explorer
	for (const auto& SourceFilePath : ResolvedFilePaths)
	{
		FPlatformProcess::ExploreFolder(*FPaths::GetPath(SourceFilePath));
	}
}

void FAssetFileContextMenu::ExecuteOpenInExternalEditor(const TArray<FString> ResolvedFilePaths)
{
	// Open all files in their respective editor
	for (const auto& SourceFilePath : ResolvedFilePaths)
	{
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*SourceFilePath, NULL, ELaunchVerb::Edit);
	}
}

void FAssetFileContextMenu::GetSelectedAssetsByClass(TMap<UClass*, TArray<FAssetData> >& OutSelectedAssetsByClass) const
{
	// Sort all selected assets by class
	for (const FAssetData& SelectedAsset : SelectedAssets)
	{
		if (UClass* AssetClass = SelectedAsset.GetClass())
		{
			TArray<FAssetData>& SelectedAssetsForClass = OutSelectedAssetsByClass.FindOrAdd(AssetClass);
			SelectedAssetsForClass.Add(SelectedAsset);
		}
	}
}

void FAssetFileContextMenu::GetSelectedAssetSourceFilePaths(TArray<FString>& OutFilePaths, TArray<FString>& OutUniqueSourceFileLabels, int32 &OutValidSelectedAssetCount) const
{
	OutFilePaths.Empty();
	OutUniqueSourceFileLabels.Empty();
	TMap<UClass*, TArray<FAssetData>> SelectedAssetsByClass;
	GetSelectedAssetsByClass(SelectedAssetsByClass);

	OutValidSelectedAssetCount = 0;

	FAssetSourceFilesArgs GetSourceFilesArgs;
	// Get the source file paths for the assets of each type
	for (const auto& AssetsByClassPair : SelectedAssetsByClass)
	{
		if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetsByClassPair.Key))
		{
			for (const FAssetData& AssetData :  AssetsByClassPair.Value)
			{
				GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&AssetData, 1);

				EAssetCommandResult Result = AssetDefinition->GetSourceFiles(GetSourceFilesArgs, [&OutFilePaths, &OutUniqueSourceFileLabels](const FAssetSourceFilesResult& AssetImportInfo)
				{
					OutFilePaths.Add(AssetImportInfo.FilePath);
					OutUniqueSourceFileLabels.AddUnique(AssetImportInfo.DisplayLabel);
					return true;
				});

				if (Result == EAssetCommandResult::Handled)
				{
					// We found some import data for the asset if the call is handled
					++OutValidSelectedAssetCount;
				}
			}
		}
	}
}

void FAssetFileContextMenu::ExecuteCreateBlueprintUsing()
{
	if(SelectedAssets.Num() == 1)
	{
		UObject* Asset = SelectedAssets[0].GetAsset();
		FKismetEditorUtilities::CreateBlueprintUsingAsset(Asset, true);
	}
}

void FAssetFileContextMenu::GetSelectedAssets(TArray<UObject*>& Assets, bool SkipRedirectors) const
{
	AssetViewUtils::FLoadAssetsSettings Settings{
		// Default settings
	};
	if (SkipRedirectors)
	{
		TArray<FAssetData> FilteredAssets;
		Algo::CopyIf(SelectedAssets, FilteredAssets, [](const FAssetData& Asset) { return !Asset.IsRedirector(); });
		AssetViewUtils::LoadAssetsIfNeeded(FilteredAssets, Assets, Settings);
	}
	else
	{
		AssetViewUtils::LoadAssetsIfNeeded(SelectedAssets, Assets, Settings);
	}
}

void FAssetFileContextMenu::GetSelectedAssetData(TArray<FAssetData>& AssetDataList, bool SkipRedirectors) const
{
	if (SkipRedirectors)
	{
		TArray<FString> SelectedAssetPaths;
		AssetDataList.Reserve(SelectedAssets.Num());
		for (const FAssetData& SelectedAsset : SelectedAssets)
		{
			if (SkipRedirectors && (SelectedAsset.AssetClassPath == UObjectRedirector::StaticClass()->GetClassPathName()))
			{
				// Don't operate on Redirectors
				continue;
			}

			AssetDataList.Add(SelectedAsset);
		}
	}
	else
	{
		AssetDataList = SelectedAssets;
	}
}

/** Generates a reference graph of the world and can then find actors referencing specified objects */
struct WorldReferenceGenerator : public FFindReferencedAssets
{
	void BuildReferencingData()
	{
		MarkAllObjects();

		const int32 MaxRecursionDepth = 0;
		const bool bIncludeClasses = true;
		const bool bIncludeDefaults = false;
		const bool bReverseReferenceGraph = true;


		UWorld* World = GWorld;

		// Generate the reference graph for the world
		FReferencedAssets* WorldReferencer = new(Referencers)FReferencedAssets(World);
		FFindAssetsArchive(World, WorldReferencer->AssetList, &ReferenceGraph, MaxRecursionDepth, bIncludeClasses, bIncludeDefaults, bReverseReferenceGraph);

		// Also include all the streaming levels in the results
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			if (StreamingLevel)
			{
				if (ULevel* Level = StreamingLevel->GetLoadedLevel())
				{
					// Generate the reference graph for each streamed in level
					FReferencedAssets* LevelReferencer = new(Referencers) FReferencedAssets(Level);			
					FFindAssetsArchive(Level, LevelReferencer->AssetList, &ReferenceGraph, MaxRecursionDepth, bIncludeClasses, bIncludeDefaults, bReverseReferenceGraph);
				}
			}
		}
	}

	void MarkAllObjects()
	{
		// Mark all objects so we don't get into an endless recursion
		for (FThreadSafeObjectIterator It; It; ++It)
		{
			It->Mark(OBJECTMARK_TagExp);
		}
	}

	void Generate( const UObject* AssetToFind, TSet<const UObject*>& OutObjects )
	{
		// Don't examine visited objects
		if (!AssetToFind->HasAnyMarks(OBJECTMARK_TagExp))
		{
			return;
		}

		AssetToFind->UnMark(OBJECTMARK_TagExp);

		// Return once we find a parent object that is an actor
		if (AssetToFind->IsA(AActor::StaticClass()))
		{
			OutObjects.Add(AssetToFind);
			return;
		}

		// Traverse the reference graph looking for actor objects
		auto* ReferencingObjects = ReferenceGraph.Find(AssetToFind);
		if (ReferencingObjects)
		{
			for (const auto& SetIt : *ReferencingObjects)
			{
				Generate(SetIt, OutObjects);
			}
		}
	}
};

void FAssetFileContextMenu::ExecuteFindAssetInWorld()
{
	TArray<UObject*> AssetsToFind;
	const bool SkipRedirectors = true;
	GetSelectedAssets(AssetsToFind, SkipRedirectors);

	const bool NoteSelectionChange = true;
	const bool DeselectBSPSurfs = true;
	const bool WarnAboutManyActors = false;
	GEditor->SelectNone(NoteSelectionChange, DeselectBSPSurfs, WarnAboutManyActors);

	if (AssetsToFind.Num() > 0)
	{
		FScopedSlowTask SlowTask(2 + AssetsToFind.Num(), NSLOCTEXT("AssetContextMenu", "FindAssetInWorld", "Finding actors that use this asset..."));
		SlowTask.MakeDialog();

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		TSet<const UObject*> OutObjects;
		WorldReferenceGenerator ObjRefGenerator;

		SlowTask.EnterProgressFrame();
		ObjRefGenerator.BuildReferencingData();

		for (UObject* AssetToFind : AssetsToFind)
		{
			SlowTask.EnterProgressFrame();
			ObjRefGenerator.MarkAllObjects();
			ObjRefGenerator.Generate(AssetToFind, OutObjects);
		}

		SlowTask.EnterProgressFrame();

		if (OutObjects.Num() > 0)
		{
			const bool InSelected = true;
			const bool Notify = false;

			// Select referencing actors
			for (const UObject* Object : OutObjects)
			{
				GEditor->SelectActor(const_cast<AActor*>(CastChecked<AActor>(Object)), InSelected, Notify);
			}

			GEditor->NoteSelectionChange();
		}
		else
		{
			FNotificationInfo Info(LOCTEXT("NoReferencingActorsFound", "No actors found."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
}

void FAssetFileContextMenu::ExecutePropertyMatrix()
{
	TArray<UObject*> ObjectsForPropertiesMenu;
	const bool SkipRedirectors = true;
	GetSelectedAssets(ObjectsForPropertiesMenu, SkipRedirectors);

	if ( ObjectsForPropertiesMenu.Num() > 0 )
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
		PropertyEditorModule.CreatePropertyEditorToolkit(TSharedPtr<IToolkitHost>(), ObjectsForPropertiesMenu );
	}
}

void FAssetFileContextMenu::ExecuteShowAssetMetaData()
{
	for (const FAssetData& AssetData : SelectedAssets)
	{
		UObject* Asset = AssetData.GetAsset();
		if (Asset)
		{
			TMap<FName, FString>* TagValues = FMetaData::GetMapForObject(Asset);
			if (TagValues)
			{
				// Create and display a resizable window to display the MetaDataView for each asset with metadata
				FString Title = FString::Printf(TEXT("Metadata: %s"), *AssetData.AssetName.ToString());

				TSharedPtr< SWindow > Window = SNew(SWindow)
					.Title(FText::FromString(Title))
					.SupportsMaximize(false)
					.SupportsMinimize(false)
					.MinWidth(500.0f)
					.MinHeight(250.0f)
					[
						SNew(SBorder)
						.Padding(4.f)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SMetaDataView, *TagValues)
						]
					];

				FSlateApplication::Get().AddWindow(Window.ToSharedRef());
			}
			else
			{
				FNotificationInfo Info(FText::Format(LOCTEXT("NoMetaDataFound", "No metadata found for asset {0}."), FText::FromString(Asset->GetName())));
				Info.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
		}
	}
}

void FAssetFileContextMenu::ExecuteCopyAssetRegistryTags()
{
	const TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	for (const FAssetData& AssetData : SelectedAssets)
	{
		TSharedPtr<FJsonObject> AssetObject;

		if (SelectedAssets.Num() != 1)
		{
			AssetObject = MakeShared<FJsonObject>();
			JsonObject->SetObjectField(AssetData.PackageName.ToString(), AssetObject);
		}
		else
		{
			AssetObject = JsonObject;
		}

		AssetData.EnumerateTags([&AssetObject](const TPair<FName, FAssetTagValueRef>& TagPair)
		{
			AssetObject->SetStringField(TagPair.Key.ToString(), TagPair.Value.AsString());
		});
	}

	FString JsonString;
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString);
	FJsonSerializer::Serialize(JsonObject, JsonWriter);

	FPlatformApplicationMisc::ClipboardCopy(*JsonString);

	FNotificationInfo Info(LOCTEXT("CopyAssetRegistryTagsNotification", "Asset Registry Tags have been copied to clipboard."));
	Info.ExpireDuration = 3.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
}

bool FAssetFileContextMenu::CanModifyPath(const FString& InPath) const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(InPath))
	{
		return false;
	}

	return true;
}

void FAssetFileContextMenu::ExecuteDiffSelected() const
{
	if (SelectedAssets.Num() >= 2)
	{
		UObject* FirstObjectSelected = SelectedAssets[0].GetAsset();
		UObject* SecondObjectSelected = SelectedAssets[1].GetAsset();

		if ((FirstObjectSelected != NULL) && (SecondObjectSelected != NULL))
		{
			// Load the asset registry module
			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

			FRevisionInfo CurrentRevision; 
			CurrentRevision.Revision = TEXT("");

			AssetToolsModule.Get().DiffAssets(FirstObjectSelected, SecondObjectSelected, CurrentRevision, CurrentRevision);
		}
	}
}

bool FAssetFileContextMenu::CanExecuteLoad() const
{
	return SelectedAssets.Num() > 0;
}

void FAssetFileContextMenu::ExecuteLoad()
{
	const bool bFullyLoad = FSlateApplication::Get().GetModifierKeys().IsShiftDown();

	FScopedSlowTask SlowTask(SelectedAssets.Num(), LOCTEXT("LoadingSelectedAssets", "Loading Selected Assets..."));
	SlowTask.MakeDialogDelayed(1.0f);

	TArray<UObject*> LoadedAssets;
	for (const FAssetData& SelectedAsset : SelectedAssets)
	{
		if (UObject* LoadedAsset = SelectedAsset.GetAsset())
		{
			LoadedAssets.Add(LoadedAsset);
		}
		SlowTask.EnterProgressFrame(1);
	}

	if (bFullyLoad)
	{
		UEditorLoadingAndSavingUtils::FullyLoadAssets(LoadedAssets);
	}
}

bool FAssetFileContextMenu::CanExecuteReload() const
{
	return SelectedAssets.Num() > 0;
}

static void ProcessObjectForUncook(const UObject* Obj, TArray<UPackage*>& PackagesToReload)
{
	if (Obj == nullptr)
	{
		return;
	}

	// tell package stores to stop serving cooked packages
	FHybridPackageStoreBackend::ForceLoadPackageAsLoose(Obj->GetPackage()->GetPackageId());

	UPackage* Pkg = Obj->GetPackage();
	if (Pkg->HasAnyPackageFlags(PKG_Cooked))
	{
		PackagesToReload.AddUnique(Pkg);
	}

	// if the object needs anymore objects to also be unloaded, call this function recursively
	// here, with those objects
}

static void ReloadPackages(TArray<FAssetData>& Assets, bool bUncookPackages)
{
	// Don't allow asset reload during PIE
	if (GIsEditor)
	{
		UEditorEngine* Editor = GEditor;
		FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
		if (PIEWorldContext)
		{
			FNotificationInfo Notification(LOCTEXT("CannotReloadAssetInPIE", "Assets cannot be reloaded while in PIE."));
			Notification.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Notification);
			return;
		}
	}

	if (Assets.Num() > 0)
	{
		TArray<UPackage*> PackagesToReload;

		for (auto AssetIt = Assets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& AssetData = *AssetIt;

			if (AssetData.AssetClassPath == UObjectRedirector::StaticClass()->GetClassPathName())
			{
				// Don't operate on Redirectors
				continue;
			}

			if (AssetData.AssetClassPath == UUserDefinedStruct::StaticClass()->GetClassPathName())
			{
				FNotificationInfo Notification(LOCTEXT("CannotReloadUserStruct", "User created structures cannot be safely reloaded."));
				Notification.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Notification);
				continue;
			}

			if (AssetData.AssetClassPath == UUserDefinedEnum::StaticClass()->GetClassPathName())
			{
				FNotificationInfo Notification(LOCTEXT("CannotReloadUserEnum", "User created enumerations cannot be safely reloaded."));
				Notification.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Notification);
				continue;
			}

			// simple case of just reloading
			if (!bUncookPackages)
			{
				PackagesToReload.AddUnique(AssetData.GetPackage());
			}
			// complexity for uncooked a packages and it's dependencies
			else
			{
				// if the object wasn't loaded, maybe it's because it was cooked, so reload the whole package uncooked,
				// then we can try again for it's related objects
				UObject* AssetObj = AssetData.FastGetAsset();
				if (AssetObj == nullptr)
				{
					UPackageTools::ReloadPackages({ AssetData.GetPackage() } );
				}

				ProcessObjectForUncook(AssetData.FastGetAsset(), PackagesToReload);
			}
		}

		if (PackagesToReload.Num() > 0)
		{
			UPackageTools::ReloadPackages(PackagesToReload);
		}
	}
}

void FAssetFileContextMenu::ExecuteReload()
{
	ReloadPackages(SelectedAssets, /*bUncook*/false);
}

bool FAssetFileContextMenu::CanExecuteReloadUncooked() const
{
	return SelectedAssets.Num() > 0;
}

void FAssetFileContextMenu::ExecuteReloadUncooked()
{
	ReloadPackages(SelectedAssets, /*bUncook*/true);
	OnRefreshView.ExecuteIfBound();
}


void FAssetFileContextMenu::ExecuteConsolidate()
{
	TArray<UObject*> ObjectsToConsolidate;
	const bool SkipRedirectors = true;
	GetSelectedAssets(ObjectsToConsolidate, SkipRedirectors);

	if ( ObjectsToConsolidate.Num() >  0 )
	{
		FConsolidateToolWindow::AddConsolidationObjects( ObjectsToConsolidate );
	}
}

void FAssetFileContextMenu::ExecuteCaptureThumbnail()
{
	FViewport* Viewport = GEditor->GetActiveViewport();

	if ( ensure(GCurrentLevelEditingViewportClient) && ensure(Viewport) )
	{
		//have to re-render the requested viewport
		FLevelEditorViewportClient* OldViewportClient = GCurrentLevelEditingViewportClient;
		//remove selection box around client during render
		GCurrentLevelEditingViewportClient = NULL;
		Viewport->Draw();

		AssetViewUtils::CaptureThumbnailFromViewport(Viewport, SelectedAssets);

		//redraw viewport to have the yellow highlight again
		GCurrentLevelEditingViewportClient = OldViewportClient;
		Viewport->Draw();
	}
}

void FAssetFileContextMenu::ExecuteClearThumbnail()
{
	AssetViewUtils::ClearCustomThumbnails(SelectedAssets);
}

void FAssetFileContextMenu::ExecuteMigrateAsset()
{
	// Get a list of package names for input into MigratePackages
	TArray<FName> PackageNames;
	PackageNames.Reserve(SelectedAssets.Num());
	for (int32 AssetIdx = 0; AssetIdx < SelectedAssets.Num(); ++AssetIdx)
	{
		PackageNames.Add(SelectedAssets[AssetIdx].PackageName);
	}

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().MigratePackages( PackageNames );
}

void FAssetFileContextMenu::ExecuteGoToCodeForAsset(UClass* SelectedClass)
{
	if (SelectedClass)
	{
		FString ClassHeaderPath;
		if( FSourceCodeNavigation::FindClassHeaderPath( SelectedClass, ClassHeaderPath ) && IFileManager::Get().FileSize( *ClassHeaderPath ) != INDEX_NONE )
		{
			const FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassHeaderPath);
			FSourceCodeNavigation::OpenSourceFile( AbsoluteHeaderPath );
		}
	}
}

void FAssetFileContextMenu::ExecuteGoToDocsForAsset(UClass* SelectedClass)
{
	ExecuteGoToDocsForAsset(SelectedClass, FString());
}

void FAssetFileContextMenu::ExecuteGoToDocsForAsset(UClass* SelectedClass, const FString ExcerptSection)
{
	if (SelectedClass)
	{
		FString DocumentationLink = FEditorClassUtils::GetDocumentationLink(SelectedClass, ExcerptSection);
		if (!DocumentationLink.IsEmpty())
		{
			FString DocumentationLinkBaseUrl = FEditorClassUtils::GetDocumentationLinkBaseUrl(SelectedClass);
			IDocumentation::Get()->Open(DocumentationLink, FDocumentationSourceInfo(TEXT("cb_docs")), DocumentationLinkBaseUrl);
		}
	}
}

#if USE_STABLE_LOCALIZATION_KEYS
void FAssetFileContextMenu::ExecuteResetLocalizationId()
{
	const FText ResetLocalizationIdMsg = LOCTEXT("ResetLocalizationIdMsg", "This will reset the localization ID of the selected assets and cause all text within them to lose their existing translations.\n\nAre you sure you want to do this?");
	if (FMessageDialog::Open(EAppMsgType::YesNo, ResetLocalizationIdMsg) != EAppReturnType::Yes)
	{
		return;
	}

	for (const FAssetData& AssetData : SelectedAssets)
	{
		UObject* Asset = AssetData.GetAsset();
		if (Asset)
		{
			if (!TextNamespaceUtil::HasDeterministicPackageNamespace(Asset))
			{
				Asset->Modify();
				TextNamespaceUtil::ResetPackageNamespace(Asset);
			}
		}
	}
}
#endif // USE_STABLE_LOCALIZATION_KEYS

void FAssetFileContextMenu::ExecuteShowLocalizationCache(const FAssetData InAsset, const FString InPackageFilename)
{
	TStringBuilder<2048> LocalizationCacheStr;
	auto AppendLocalizationCache = [&LocalizationCacheStr](const FString& CachedLocalizationId, const TArray<FGatherableTextData>& GatherableTextDataArray)
	{
		// Convert the gathered text array into a readable format
		LocalizationCacheStr.Appendf(TEXT("Package: %s"), *CachedLocalizationId);
		for (const FGatherableTextData& GatherableTextData : GatherableTextDataArray)
		{
			if (LocalizationCacheStr.Len() > 0)
			{
				LocalizationCacheStr += TEXTVIEW("\n\n");
			}

			TStringBuilder<512> KeysStr;
			TStringBuilder<512> EditorOnlyKeysStr;
			for (const FTextSourceSiteContext& TextSourceSiteContext : GatherableTextData.SourceSiteContexts)
			{
				TStringBuilder<512>* KeysStrPtr = TextSourceSiteContext.IsEditorOnly ? &EditorOnlyKeysStr : &KeysStr;
				if (KeysStrPtr->Len() > 0)
				{
					*KeysStrPtr += TEXTVIEW(", ");
				}
				*KeysStrPtr += TextSourceSiteContext.KeyName;
			}

			LocalizationCacheStr.Appendf(TEXT("Namespace: %s\n"), *GatherableTextData.NamespaceName);
			if (KeysStr.Len() > 0)
			{
				LocalizationCacheStr.Appendf(TEXT("Keys: %s\n"), *KeysStr);
			}
			if (EditorOnlyKeysStr.Len() > 0)
			{
				LocalizationCacheStr.Appendf(TEXT("Keys (Editor-Only): %s\n"), *EditorOnlyKeysStr);
			}
			LocalizationCacheStr.Appendf(TEXT("Source: %s"), *GatherableTextData.SourceData.SourceString);
		}
	};

	// Read the localization data from the cache in the package header
	{
		FString CachedLocalizationId;
		TArray<FGatherableTextData> GatherableTextDataArray;

		TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InPackageFilename));
		if (FileReader)
		{
			// Read package file summary from the file
			FPackageFileSummary PackageFileSummary;
			*FileReader << PackageFileSummary;
			FileReader->SetCustomVersions(PackageFileSummary.GetCustomVersionContainer());

			CachedLocalizationId = PackageFileSummary.LocalizationId;

			if (PackageFileSummary.GatherableTextDataOffset > 0)
			{
				FileReader->Seek(PackageFileSummary.GatherableTextDataOffset);

				GatherableTextDataArray.SetNum(PackageFileSummary.GatherableTextDataCount);
				for (int32 GatherableTextDataIndex = 0; GatherableTextDataIndex < PackageFileSummary.GatherableTextDataCount; ++GatherableTextDataIndex)
				{
					*FileReader << GatherableTextDataArray[GatherableTextDataIndex];
				}
			}
		}

		LocalizationCacheStr += TEXTVIEW("== On-Disk Cache ==\n\n");
		AppendLocalizationCache(CachedLocalizationId, GatherableTextDataArray);
	}

	// If the asset is loaded, also show the "live" version of the cache (for debugging cache invalidation issues)
	{
		LocalizationCacheStr += TEXTVIEW("\n\n\n\n== In-Memory Cache ==\n\n");

		if (UObject* Asset = InAsset.FastGetAsset())
		{
			TArray<FGatherableTextData> GatherableTextDataArray;

			EPropertyLocalizationGathererResultFlags GatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;
			FPropertyLocalizationDataGatherer _(GatherableTextDataArray, Asset->GetPackage(), GatherableTextResultFlags);

#if USE_STABLE_LOCALIZATION_KEYS
			const FString CachedLocalizationId = TextNamespaceUtil::GetPackageNamespace(Asset);
			AppendLocalizationCache(CachedLocalizationId, GatherableTextDataArray);
#endif // USE_STABLE_LOCALIZATION_KEYS
		}
		else
		{
			LocalizationCacheStr += TEXTVIEW("<Asset Not Loaded>");
		}
	}

	// Generate a message box for the result
	SGenericDialogWidget::OpenDialog(LOCTEXT("LocalizationCache", "Localization Cache"), 
		SNew(SBox)
		.MaxDesiredWidth(800.0f)
		.MaxDesiredHeight(400.0f)
		[
			SNew(SMultiLineEditableTextBox)
			.IsReadOnly(true)
			.AutoWrapText(true)
			.Text(FText::AsCultureInvariant(LocalizationCacheStr))
		],
		SGenericDialogWidget::FArguments()
		.UseScrollBox(false)
	);
}

void FAssetFileContextMenu::ExecuteExport()
{
	TArray<UObject*> ObjectsToExport;
	const bool SkipRedirectors = false;
	GetSelectedAssets(ObjectsToExport, SkipRedirectors);

	if ( ObjectsToExport.Num() > 0 )
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		AssetToolsModule.Get().ExportAssetsWithDialog(ObjectsToExport, true);
	}
}

void FAssetFileContextMenu::ExecuteBulkExport()
{
	TArray<UObject*> ObjectsToExport;
	const bool SkipRedirectors = false;
	GetSelectedAssets(ObjectsToExport, SkipRedirectors);

	if ( ObjectsToExport.Num() > 0 )
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		AssetToolsModule.Get().ExportAssetsWithDialog(ObjectsToExport, false);
	}
}

bool FAssetFileContextMenu::CanExecuteCreateBlueprintUsing() const
{
	// Only work if you have a single asset selected
	if (SelectedAssets.Num() == 1)
	{
		// There's no need to resolve the class, we don't need to probably worry about blueprint components here for
		// the component asset brokerage, so just loaded native classes will work.
		if (UClass* AssetClass = SelectedAssets[0].GetClass(EResolveClass::No))
		{
			// Just see if there's a primary component for this asset class.
			return FComponentAssetBrokerage::GetPrimaryComponentForAsset(AssetClass) != nullptr;
		}
	}

	return false;
}

bool FAssetFileContextMenu::CanExecuteFindAssetInWorld() const
{
	return bAtLeastOneNonRedirectorSelected;
}

bool FAssetFileContextMenu::CanExecuteProperties() const
{
	return bAtLeastOneNonRedirectorSelected;
}

bool FAssetFileContextMenu::CanExecutePropertyMatrix(FText& OutErrorMessage) const
{
	bool bResult = bAtLeastOneNonRedirectorSelected;
	if (bAtLeastOneNonRedirectorSelected)
	{
		const bool SkipRedirectors = true;
		TArray<FAssetData> AssetDataList;
		GetSelectedAssetData(AssetDataList, SkipRedirectors);

		// Ensure all Blueprints are valid.
		static FName GeneratedClassName = TEXT("GeneratedClass");
		for (const FAssetData& AssetData : AssetDataList)
		{
			FString GeneratedClassValue;
			if (AssetData.GetTagValue(GeneratedClassName, GeneratedClassValue))
			{
				if (GeneratedClassValue.IsEmpty() || GeneratedClassValue == TEXT("None"))
				{
					OutErrorMessage = LOCTEXT("InvalidBlueprint", "A selected Blueprint is invalid.");
					bResult = false;
					break;
				}
			}
		}
	}
	return bResult;
}

bool FAssetFileContextMenu::CanExecutePropertyMatrix() const
{
	FText ErrorMessageDummy;
	return CanExecutePropertyMatrix(ErrorMessageDummy);
}

FText FAssetFileContextMenu::GetExecutePropertyMatrixTooltip() const
{
	FText ResultTooltip;
	if (CanExecutePropertyMatrix(ResultTooltip))
	{
		ResultTooltip = LOCTEXT("PropertyMatrixTooltip", "Bulk edit the selected assets in the Property Matrix");
	}
	return ResultTooltip;
}

bool FAssetFileContextMenu::CanExecuteCaptureThumbnail() const
{
	return GCurrentLevelEditingViewportClient != NULL;
}

bool FAssetFileContextMenu::CanClearCustomThumbnails() const
{
	for ( auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt )
	{
		if ( AssetViewUtils::AssetHasCustomThumbnail(*AssetIt) )
		{
			return true;
		}
	}

	return false;
}

void FAssetFileContextMenu::CacheCanExecuteVars()
{
	bAtLeastOneNonRedirectorSelected = false;

	for (auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt)
	{
		const FAssetData& AssetData = *AssetIt;
		if ( !AssetData.IsValid() )
		{
			continue;
		}

		if ( !bAtLeastOneNonRedirectorSelected && AssetData.AssetClassPath != UObjectRedirector::StaticClass()->GetClassPathName() )
		{
			bAtLeastOneNonRedirectorSelected = true;
		}

		if ( bAtLeastOneNonRedirectorSelected
			)
		{
			// All options are available, no need to keep iterating
			break;
		}
	}
}

void FAssetFileContextMenu::GetSelectedPackageNames(TArray<FString>& OutPackageNames) const
{
	for (int32 AssetIdx = 0; AssetIdx < SelectedAssets.Num(); ++AssetIdx)
	{
		OutPackageNames.Add(SelectedAssets[AssetIdx].PackageName.ToString());
	}
}

void FAssetFileContextMenu::GetSelectedPackages(TArray<UPackage*>& OutPackages) const
{
	for (int32 AssetIdx = 0; AssetIdx < SelectedAssets.Num(); ++AssetIdx)
	{
		UPackage* Package = FindPackage(NULL, *SelectedAssets[AssetIdx].PackageName.ToString());

		if ( Package )
		{
			OutPackages.Add(Package);
		}
	}
}

void FAssetFileContextMenu::MakeChunkIDListMenu(UToolMenu* Menu)
{
	TArray<int32> FoundChunks;
	for (const auto& SelectedAsset : SelectedAssets)
	{
		UPackage* Package = FindPackage(NULL, *SelectedAsset.PackageName.ToString());

		if (Package)
		{
			for (auto ChunkID : Package->GetChunkIDs())
			{
				FoundChunks.AddUnique(ChunkID);
			}
		}
	}

	FToolMenuSection& Section = Menu->AddSection("Chunks");
	for (auto ChunkID : FoundChunks)
	{
		Section.AddMenuEntry(
			NAME_None,
			FText::Format(LOCTEXT("PackageChunk", "Chunk {0}"), FText::AsNumber(ChunkID)),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetFileContextMenu::ExecuteRemoveChunkID, ChunkID)
			)
		);
	}
}

void FAssetFileContextMenu::ExecuteAssignChunkID()
{
	TSharedPtr<SWidget> ParentWidgetPtr = ParentWidget.Pin();
	if (SelectedAssets.Num() > 0 && ParentWidgetPtr)
	{
		// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
		const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
		FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);

		FVector2D AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition(Anchor, FVector2D(441, 537), true, FVector2D::ZeroVector, Orient_Horizontal);

		TSharedPtr<SWindow> Window = SNew(SWindow)
			.AutoCenter(EAutoCenter::None)
			.ScreenPosition(AdjustedSummonLocation)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::Autosized)
			.Title(LOCTEXT("WindowHeader", "Enter Chunk ID"));

		Window->SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MeshPaint_LabelStrength", "Chunk ID"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(2.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<int32>)
					.AllowSpin(true)
					.MinSliderValue(0)
					.MaxSliderValue(300)
					.MinValue(0)
					.MaxValue(300)
					.Value(this, &FAssetFileContextMenu::GetChunkIDSelection)
					.OnValueChanged(this, &FAssetFileContextMenu::OnChunkIDAssignChanged)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ChunkIDAssign_Yes", "OK"))
					.OnClicked(this, &FAssetFileContextMenu::OnChunkIDAssignCommit, Window)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ChunkIDAssign_No", "Cancel"))
					.OnClicked(this, &FAssetFileContextMenu::OnChunkIDAssignCancel, Window)
				]
			]
		);

		ChunkIDSelected = 0;
		FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), ParentWidgetPtr);
	}
}

void FAssetFileContextMenu::ExecuteRemoveAllChunkID()
{
	TArray<int32> EmptyChunks;
	for (const auto& SelectedAsset : SelectedAssets)
	{
		UPackage* Package = FindPackage(NULL, *SelectedAsset.PackageName.ToString());

		if (Package)
		{
			Package->SetChunkIDs(EmptyChunks);
			Package->SetDirtyFlag(true);
		}
	}
}

TOptional<int32> FAssetFileContextMenu::GetChunkIDSelection() const
{
	return ChunkIDSelected;
}

void FAssetFileContextMenu::OnChunkIDAssignChanged(int32 NewChunkID)
{
	ChunkIDSelected = NewChunkID;
}

FReply FAssetFileContextMenu::OnChunkIDAssignCommit(TSharedPtr<SWindow> Window)
{
	for (const auto& SelectedAsset : SelectedAssets)
	{
		UPackage* Package = FindPackage(NULL, *SelectedAsset.PackageName.ToString());

		if (Package)
		{
			TArray<int32> CurrentChunks = Package->GetChunkIDs();
			CurrentChunks.AddUnique(ChunkIDSelected);
			Package->SetChunkIDs(CurrentChunks);
			Package->SetDirtyFlag(true);
		}
	}

	Window->RequestDestroyWindow();

	return FReply::Handled();
}

FReply FAssetFileContextMenu::OnChunkIDAssignCancel(TSharedPtr<SWindow> Window)
{
	Window->RequestDestroyWindow();

	return FReply::Handled();
}

void FAssetFileContextMenu::ExecuteRemoveChunkID(int32 ChunkID)
{
	for (const auto& SelectedAsset : SelectedAssets)
	{
		UPackage* Package = FindPackage(NULL, *SelectedAsset.PackageName.ToString());

		if (Package)
		{
			int32 FoundIndex;
			TArray<int32> CurrentChunks = Package->GetChunkIDs();
			CurrentChunks.Find(ChunkID, FoundIndex);
			if (FoundIndex != INDEX_NONE)
			{
				CurrentChunks.RemoveAt(FoundIndex);
				Package->SetChunkIDs(CurrentChunks);
				Package->SetDirtyFlag(true);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
