// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetFolderContextMenu.h"
#include "ContentBrowserDataMenuContexts.h"
#include "Misc/MessageDialog.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "IAssetTools.h"
#include "Misc/ScopedSlowTask.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

void FAssetFolderContextMenu::MakeContextMenu(UToolMenu* InMenu, const TArray<FString>& InSelectedPackagePaths)
{
	MakeContextMenu(InMenu, InSelectedPackagePaths, TArray<FString>());
}

void FAssetFolderContextMenu::MakeContextMenu(UToolMenu* InMenu, const TArray<FString>& InSelectedPackagePaths, const TArray<FString>& InSelectedPackages)
{
	SelectedPaths = InSelectedPackagePaths;
	SelectedPackages = InSelectedPackages;

	if (SelectedPaths.Num() > 0)
	{
		AddMenuOptions(InMenu);
	}
}

void FAssetFolderContextMenu::AddMenuOptions(UToolMenu* Menu)
{
	UContentBrowserDataMenuContext_FolderMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
	checkf(Context, TEXT("Required context UContentBrowserDataMenuContext_FolderMenu was missing!"));

	if(Context->bCanBeModified)
	{
		// Bulk operations section //
		{
			FToolMenuSection& Section = Menu->AddSection("PathContextBulkOperations", LOCTEXT("AssetTreeBulkMenuHeading", "Bulk Operations") );

			// Fix Up Redirectors in Folder
			{
				FToolMenuEntry& Entry = Section.AddMenuEntry(
					"FixUpRedirectorsInFolder",
					LOCTEXT("FixUpRedirectorsInFolder", "Update Redirector References"),
					LOCTEXT("FixUpRedirectorsInFolderTooltip", "Finds references to all redirectors in the selected items and resaves the referencing assets if possible, so that they reference the target of the redirector directly instead."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Adjust"),
					FUIAction(FExecuteAction::CreateSP(this, &FAssetFolderContextMenu::ExecuteFixUpRedirectorsInFolder)));
				Entry.InsertPosition = FToolMenuInsert("Delete", EToolMenuInsertType::After);
			}

			if (!SelectedPaths.IsEmpty() || !SelectedPackages.IsEmpty())
			{
				// Migrate Folder
				FToolMenuEntry& Entry = Section.AddMenuEntry(
					"MigrateFolder",
					LOCTEXT("MigrateFolder", "Migrate..."),
					LOCTEXT("MigrateFolderTooltip", "Copies assets found in the selection and their dependencies to another content folder."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.Migrate"),
					FUIAction( FExecuteAction::CreateSP( this, &FAssetFolderContextMenu::ExecuteMigrateFolder ) )
					);
				Entry.InsertPosition = FToolMenuInsert("FixUpRedirectorsInFolder", EToolMenuInsertType::After);
			}
		}
	}
}

void FAssetFolderContextMenu::ExecuteMigrateFolder()
{
	const FString& SourcesPath = GetFirstSelectedPath();
	if ( ensure(SourcesPath.Len()) )
	{
		// @todo Make sure the asset registry has completed discovering assets, or else GetAssetsByPath() will not find all the assets in the folder! Add some UI to wait for this with a cancel button
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if ( AssetRegistryModule.Get().IsLoadingAssets() )
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT( "MigrateFolderAssetsNotDiscovered", "You must wait until asset discovery is complete to migrate a folder" ));
			return;
		}

		// Get a list of package names for input into MigratePackages
		TArray<FAssetData> AssetDataList;
		TArray<FName> PackageNames;
		AssetViewUtils::GetAssetsInPaths(SelectedPaths, AssetDataList);
		for ( auto AssetIt = AssetDataList.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			// Don't collect the external packages here
			if (AssetIt->GetOptionalOuterPathName().IsNone())
			{
			PackageNames.Add((*AssetIt).PackageName);
		}
		}

		PackageNames.Append(SelectedPackages);

		// Load all the assets in the selected paths
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().MigratePackages( PackageNames );
	}
}

void FAssetFolderContextMenu::ExecuteFixUpRedirectorsInFolder()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Form a filter from the paths
	FARFilter Filter;
	Filter.bRecursivePaths = true;

	Filter.PackagePaths.Reserve(SelectedPaths.Num());
	for (const FString& Path : SelectedPaths)
	{
		Filter.PackagePaths.Emplace(*Path);
	}

	if (!SelectedPaths.IsEmpty())
	{
		Filter.ClassPaths.Add(UObjectRedirector::StaticClass()->GetClassPathName());
	}
	
	// Query for a list of assets in the selected paths
	TArray<FAssetData> AssetList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetList);

	Filter.Clear();

	Filter.PackageNames.Reserve(SelectedPackages.Num());
	for (const FString& PackageName : SelectedPackages)
	{
		Filter.PackageNames.Emplace(*PackageName);
	}

	if (!SelectedPackages.IsEmpty())
	{
		Filter.ClassPaths.Add(UObjectRedirector::StaticClass()->GetClassPathName());
	}

	AssetRegistryModule.Get().GetAssets(Filter, AssetList);

	if (AssetList.Num() == 0)
	{
		return;
	}

	FScopedSlowTask SlowTask(3, LOCTEXT("FixupRedirectorsSlowTask", "Fixing up redirectors"));
	SlowTask.MakeDialog(true);

	SlowTask.EnterProgressFrame(1, LOCTEXT("FixupRedirectors_LoadAssets", "Loading Assets..."));
	TArray<UObject*> Objects;
	AssetViewUtils::FLoadAssetsSettings Settings{
		.bFollowRedirectors = false,
		.bAllowCancel = true,
	};
	AssetViewUtils::ELoadAssetsResult Result = AssetViewUtils::LoadAssetsIfNeeded(AssetList, Objects, Settings);
	if (Result != AssetViewUtils::ELoadAssetsResult::Cancelled && !SlowTask.ShouldCancel())
	{
		TArray<UObjectRedirector*> Redirectors;
		for (UObject* Object : Objects)
		{
			if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
			{
				Redirectors.Add(Redirector);
			}
		}

		SlowTask.EnterProgressFrame(1, LOCTEXT("FixupRedirectors_FixupReferencers", "Fixing up referencers..."));
		// Load the asset tools module
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		AssetToolsModule.Get().FixupReferencers(Redirectors, true, ERedirectFixupMode::PromptForDeletingRedirectors);
	}
}

FString FAssetFolderContextMenu::GetFirstSelectedPath() const
{
	return SelectedPaths.Num() > 0 ? SelectedPaths[0] : TEXT("");
}

#undef LOCTEXT_NAMESPACE
