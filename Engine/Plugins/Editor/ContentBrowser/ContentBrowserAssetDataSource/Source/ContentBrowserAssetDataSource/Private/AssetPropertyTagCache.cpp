// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPropertyTagCache.h"
#include "ContentBrowserAssetDataCore.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UnrealType.h"
#include "UObject/LinkerLoad.h"

FAssetPropertyTagCache& FAssetPropertyTagCache::Get()
{
	static FAssetPropertyTagCache AssetPropertyTagCache;
	return AssetPropertyTagCache;
}

const FAssetPropertyTagCache::FClassPropertyTagCache* FAssetPropertyTagCache::FindCacheForClass(FTopLevelAssetPath InClassName)
{
	FReadScopeLock ReadScope(Lock);
	TSharedPtr<FClassPropertyTagCache> ClassCache = ClassToCacheMap.FindRef(InClassName);
	return ClassCache.Get();
}

const FAssetPropertyTagCache::FClassPropertyTagCache& FAssetPropertyTagCache::GetCacheForClass(FTopLevelAssetPath InClassName)
{
	TSharedPtr<FClassPropertyTagCache> ClassCache;
	{
		FReadScopeLock ReadScope(Lock);
		ClassCache = ClassToCacheMap.FindRef(InClassName);
	}
	if (ClassCache.IsValid())
	{
		return *ClassCache;
	}
	TryCacheClass(InClassName);
	{
		FReadScopeLock ReadScope(Lock);
		ClassCache = ClassToCacheMap.FindRef(InClassName);
	}
	checkf(ClassCache.IsValid(), TEXT("Failed to create property tag cache for class %s"), *WriteToString<256>(InClassName));
	return *ClassCache;
}

void FAssetPropertyTagCache::CachePendingClasses()
{
	TSet<FTopLevelAssetPath> PendingCopy;
	{ 
		FWriteScopeLock WriteScope(Lock);
		PendingCopy = PendingClasses;
	}

	for (FTopLevelAssetPath Path : PendingCopy)
	{
		TryCacheClass(Path);
	}
}

void FAssetPropertyTagCache::TryCacheClass(FTopLevelAssetPath InClassName)
{
	TSharedPtr<FClassPropertyTagCache> ClassCache;
	
	{
		FReadScopeLock ReadScope(Lock);
		ClassCache = ClassToCacheMap.FindRef(InClassName);
	}

	if (ClassCache.IsValid())
	{
		return;
	}

	auto GetAssetClass = [&InClassName]()
	{
		UClass* FoundClass = FindObject<UClass>(InClassName);

		if (!FoundClass)
		{
			// Look for class redirectors
			FString NewPath = FLinkerLoad::FindNewPathNameForClass(InClassName.ToString(), false);
			if (!NewPath.IsEmpty())
			{
				FoundClass = FindObject<UClass>(nullptr, *NewPath);
			}
		}

		return FoundClass;
	};

	if (UClass* AssetClass = GetAssetClass())
	{
		// On a miss, race with other potential creations rather than hold the lock against queries for another class
		UE_LOGF(LogContentBrowserAssetDataSource, Verbose, "Creating asset property tag cache for %ls", *WriteToString<256>(InClassName));

		ClassCache = MakeShared<FClassPropertyTagCache>();
		FTopLevelAssetPath ActualClassPath(AssetClass);

		// Get the tags data from the CDO and use it to build the cache
		UObject* AssetClassDefaultObject = AssetClass->GetDefaultObject();
		FAssetRegistryTagsContextData TagsContext(AssetClassDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
		AssetClassDefaultObject->GetAssetRegistryTags(TagsContext);
		TMap<FName, UObject::FAssetRegistryTagMetadata> AssetTagMetaData;
		AssetClassDefaultObject->GetAssetRegistryTagMetadata(AssetTagMetaData);

		ClassCache->TagNameToCachedDataMap.Reserve(TagsContext.Tags.Num());
		for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
		{
			FPropertyTagCache& TagCache = ClassCache->TagNameToCachedDataMap.Add(TagPair.Key);
			TagCache.TagType = TagPair.Value.Type;
			TagCache.DisplayFlags = TagPair.Value.DisplayFlags;

			if (const UObject::FAssetRegistryTagMetadata* TagMetaData = AssetTagMetaData.Find(TagPair.Key))
			{
				TagCache.DisplayName = TagMetaData->DisplayName;
				TagCache.TooltipText = TagMetaData->TooltipText;
				TagCache.Suffix = TagMetaData->Suffix;
				TagCache.ImportantValue = TagMetaData->ImportantValue;
			}
			else
			{
				// If the tag name corresponds to a property name, use the property tooltip
				const FProperty* Property = FindFProperty<FProperty>(AssetClass, TagPair.Key);
				TagCache.TooltipText = Property ? Property->GetToolTipText() : FText::FromString(FName::NameToDisplayString(TagPair.Key.ToString(), false));
			}

			// Ensure a display name for this tag
			if (TagCache.DisplayName.IsEmpty())
			{
				if (const FProperty* TagField = FindFProperty<FProperty>(AssetClass, TagPair.Key))
				{
					// Take the display name from the corresponding property if possible
					TagCache.DisplayName = TagField->GetDisplayNameText();
				}
				else
				{
					// We have no type information by this point, so no idea if it's a bool :(
					TagCache.DisplayName = FText::AsCultureInvariant(FName::NameToDisplayString(TagPair.Key.ToString(), /*bIsBool*/false));
				}
			}

			// Add a mapping from the sanitized display name back to the internal tag name
			// This is useful as people only see the display name in the UI, so are more likely to use it
			{
				const FName SanitizedDisplayName = MakeObjectNameFromDisplayLabel(TagCache.DisplayName.ToString(), NAME_None);
				if (TagPair.Key != SanitizedDisplayName)
				{
					ClassCache->DisplayNameToTagNameMap.Add(SanitizedDisplayName, TagPair.Key);
				}
			}
		}

		FWriteScopeLock WriteScope(Lock);
		ClassToCacheMap.Add(ActualClassPath, ClassCache);
		PendingClasses.Remove(ActualClassPath);
		if (InClassName != ActualClassPath)
		{
			ClassToCacheMap.Add(InClassName, ClassCache);
			PendingClasses.Remove(InClassName);
		}
	}
	else 
	{
		// Some assets may be scanned before their modules are initialized
		FWriteScopeLock WriteScope(Lock);
		PendingClasses.Add(InClassName);
	}
}
