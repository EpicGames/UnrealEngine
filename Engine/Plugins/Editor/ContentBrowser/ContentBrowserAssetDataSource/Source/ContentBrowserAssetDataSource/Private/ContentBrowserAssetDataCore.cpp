// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAssetDataCore.h"

#include "AssetDefinition.h"
#include "AssetFileContextMenu.h"
#include "AssetFolderContextMenu.h"
#include "AssetPropertyTagCache.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetViewUtils.h"
#include "ContentBrowserAssetDataPayload.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDataUtils.h"
#include "ContentBrowserItemPath.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/Level.h"
#include "FileHelpers.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "IContentBrowserDataModule.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/WarnIfAssetsLoadedInScope.h"
#include "ObjectTools.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenu.h"
#include "Virtualization/VirtualizationSystem.h"

#define LOCTEXT_NAMESPACE "ContentBrowserAssetDataSource"

DEFINE_LOG_CATEGORY(LogContentBrowserAssetDataSource);

namespace ContentBrowserAssetData
{

FContentBrowserItemData CreateAssetFolderItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalFolderPath,
	const bool bIsCookedPath,
	const bool bIsPlugin)
{
	const FString FolderItemName = FPackageName::GetShortName(InInternalFolderPath);
	FText FolderDisplayNameOverride = ContentBrowserDataUtils::GetFolderItemDisplayNameOverride(
		InInternalFolderPath, FolderItemName, /*bIsClassesFolder*/ false, bIsCookedPath);
	return FContentBrowserItemData(
		InOwnerDataSource,
		EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Asset
			| (bIsPlugin ? EContentBrowserItemFlags::Category_Plugin : EContentBrowserItemFlags::None),
		InVirtualPath,
		*FolderItemName,
		MoveTemp(FolderDisplayNameOverride),
		MakeShared<FContentBrowserAssetFolderItemDataPayload>(InInternalFolderPath),
		{ InInternalFolderPath });
}

FContentBrowserItemData CreateAssetFileItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalPath,
	const FAssetData& InAssetData,
	const bool bIsPluginAsset)
{
	return FContentBrowserItemData(
		InOwnerDataSource,
		EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Asset
			| (bIsPluginAsset ? EContentBrowserItemFlags::Category_Plugin : EContentBrowserItemFlags::None),
		InVirtualPath,
		InAssetData.AssetName,
		FText(),
		MakeShared<FContentBrowserAssetFileItemDataPayload>(InAssetData),
		{ InInternalPath });
}

FContentBrowserItemData CreateUnsupportedAssetFileItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalPath,
	const FAssetData& InAssetData)
{
	return FContentBrowserItemData(
		InOwnerDataSource,
		EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Asset
			| EContentBrowserItemFlags::Misc_Unsupported,
		InVirtualPath,
		InAssetData.AssetName,
		FText(),
		MakeShared<FContentBrowserUnsupportedAssetFileItemDataPayload>(InAssetData),
		{ InInternalPath });
}


TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> GetAssetFolderItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource
		// If both these flags are not present, it's a virtual folder
		&& EnumHasAllFlags(InItem.GetItemFlags(), EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Asset)
		&& InItem.IsSupported())
	{
		return StaticCastSharedPtr<const FContentBrowserAssetFolderItemDataPayload>(InItem.GetPayload());
	}
	return nullptr;
}

TSharedPtr<const FContentBrowserAssetFileItemDataPayload> GetAssetFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource && InItem.IsFile() && InItem.IsSupported())
	{
		return StaticCastSharedPtr<const FContentBrowserAssetFileItemDataPayload>(InItem.GetPayload());
	}
	return nullptr;
}

TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> GetUnsupportedAssetFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource && EnumHasAllFlags(InItem.GetItemFlags(), EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Misc_Unsupported))
	{
		return StaticCastSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload>(InItem.GetPayload());
	}

	return nullptr;
}

void EnumerateAssetFolderItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>&)> InFolderPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, Item))
		{
			if (!InFolderPayloadCallback(FolderPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void EnumerateAssetFileItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFileItemDataPayload>&)> InAssetPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InAssetPayloadCallback(AssetPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void EnumerateAssetItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFileItemDataPayload>&)> InAssetPayloadCallback)
{
	EnumerateAssetItemPayloads(InOwnerDataSource, InItems, InFolderPayloadCallback, InAssetPayloadCallback, [](const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>&){ return true; });
}

void EnumerateAssetItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFileItemDataPayload>&)> InAssetPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>&)> InUnsupportedAssetPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, Item))
		{
			if (!InFolderPayloadCallback(FolderPayload.ToSharedRef()))
			{
				break;
			}
		}

		if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InAssetPayloadCallback(AssetPayload.ToSharedRef()))
			{
				break;
			}
		}

		if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InUnsupportedAssetPayloadCallback(UnsupportedAssetPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

bool IsPrimaryAsset(const FAssetData& InAssetData)
{
	return ContentBrowserDataUtils::IsPrimaryAsset(InAssetData);
}

bool IsPrimaryAsset(UObject* InObject)
{
	return ContentBrowserDataUtils::IsPrimaryAsset(InObject);
}

void SetOptionalErrorMessage(FText* OutErrorMsg, FText InErrorMsg)
{
	if (OutErrorMsg)
	{
		*OutErrorMsg = MoveTemp(InErrorMsg);
	}
}

bool CanModifyPath(IAssetTools* InAssetTools, const FName InFolderPath, FText* OutErrorMsg)
{
	const TSharedRef<FPathPermissionList>& WritableFolderFilter = InAssetTools->GetWritableFolderPermissionList();
	if (!WritableFolderFilter->PassesStartsWithFilter(InFolderPath))
	{
		SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsLocked", "Folder '{0}' is read only and its contents cannot be edited"), FText::FromName(InFolderPath)));
		return false;
	}
	return true;
}

bool CanModifyItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return CanModifyAssetFolderItem(InAssetTools, *FolderPayload, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanModifyAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanModifyAssetFolderItem(IAssetTools* InAssetTools, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg)
{
	return CanModifyPath(InAssetTools, InFolderPayload.GetInternalPath(), OutErrorMsg);
}

bool CanModifyAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	if (!CanModifyPath(InAssetTools, InAssetPayload.GetAssetData().PackageName, OutErrorMsg))
	{
		return false;
	}

	if (const UClass* AssetClass = InAssetPayload.GetAssetData().GetClass())
	{
		if (AssetClass->IsChildOf<UClass>())
		{
			SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotModifyGeneratedClasses", "Cannot modify generated classes"));
			return false;
		}
	}

	if (InAssetPayload.GetAssetData().HasAnyPackageFlags(PKG_Cooked | PKG_FilterEditorOnly))
	{
		SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotModifyCookedAssets", "Cannot modify cooked assets"));
		return false;
	}

	return true;
}

bool CanEditItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanEditAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanEditAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	if (UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem())
	{
		const TSharedRef<FPathPermissionList>& EditableFolderFilter = ContentBrowserDataSubsystem->GetEditableFolderPermissionList();
		FName AssetPackageName = InAssetPayload.GetAssetData().PackageName;

		if (!EditableFolderFilter->PassesStartsWithFilter(AssetPackageName))
		{
			SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsNotEditable", "Asset '{0}' is in a folder that does not allow edits. Unable to edit read only assets."), FText::FromName(AssetPackageName)));
			return false;
		}
	}

	if (!CanModifyAssetFileItem(InAssetTools, InAssetPayload, OutErrorMsg))
	{
		return false;
	}

	return true;
}

bool CanViewItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanViewAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanViewAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	const TWeakPtr<IAssetTypeActions> AssetTypeActions = InAssetTools->GetAssetTypeActionsForClass(InAssetPayload.GetAssetData().GetClass());
	if (AssetTypeActions.IsValid())
	{
		return AssetTypeActions.Pin()->SupportsOpenedMethod(EAssetTypeActivationOpenedMethod::View);
	}
	return false;
}

bool CanPreviewItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanPreviewAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanPreviewAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	return true;
}

bool EditOrPreviewAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, EAssetTypeActivationMethod::Type ActivationMethod, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	if (InAssetPayloads.Num() == 0)
	{
		return false;
	}

	ensure(ActivationMethod != EAssetTypeActivationMethod::Type::DoubleClicked);

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

	TMap<TSharedPtr<IAssetTypeActions>, TArray<FAssetData>> TypeActionsToAssetData;

	const FText DefaultText = InAssetPayloads.Num() == 1
		? FText::Format(LOCTEXT("LoadingAssetName", "Loading {0}..."), FText::FromName(InAssetPayloads[0]->GetAssetData().AssetName))
		: FText::Format(LOCTEXT("LoadingXAssets", "Loading {0} {0}|plural(one=Asset,other=Assets)..."), InAssetPayloads.Num());

	FScopedSlowTask SlowTask(100, DefaultText);
	SlowTask.MakeDialogDelayed(0.1f);

	// Iterate over all activated assets to map them to AssetTypeActions.
	// This way individual asset type actions will get a batched list of assets to operate on
	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : InAssetPayloads)
	{
		const FAssetData& AssetData = AssetPayload->GetAssetData();
		TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetPayload->GetAssetTypeActions();
		TArray<FAssetData>& AssetList = TypeActionsToAssetData.FindOrAdd(AssetTypeActions);
		AssetList.AddUnique(AssetData);
	}

	// Now that we have created our map, load and activate all the lists of objects for each asset type action.
	const bool bHasOpenActivationMethod = (ActivationMethod == EAssetTypeActivationMethod::DoubleClicked || ActivationMethod == EAssetTypeActivationMethod::Opened);
	bool bSuccessfulEditorOpen = true;
	for (auto& TypeActionToObjectsPair : TypeActionsToAssetData)
	{
		SlowTask.EnterProgressFrame(25.0f / TypeActionsToAssetData.Num());

		const TSharedPtr<IAssetTypeActions> TypeActions = TypeActionToObjectsPair.Key;
		TArray<FAssetData>& AssetsToLoad = TypeActionToObjectsPair.Value;
		if (TypeActions.IsValid())
		{
			AssetsToLoad = TypeActions->GetValidAssetsForPreviewOrEdit(AssetsToLoad, ActivationMethod == EAssetTypeActivationMethod::Type::Previewed);
		}

		TArray<UObject*> ObjList;
		ObjList.Reserve(AssetsToLoad.Num());
		
		for (const FAssetData& AssetData : AssetsToLoad)
		{
			SlowTask.EnterProgressFrame(75.0f / InAssetPayloads.Num(), FText::Format(LOCTEXT("LoadingAssetName", "Loading {0}..."), FText::FromName(AssetData.AssetName)));

			ObjList.Add(AssetData.GetAsset());
		}

		bool bOpenEditorForAssets = bHasOpenActivationMethod;
		if (TypeActions.IsValid())
		{
			bOpenEditorForAssets = !TypeActions->AssetsActivatedOverride(ObjList, ActivationMethod);
		}

		if (bOpenEditorForAssets)
		{
			bSuccessfulEditorOpen &= AssetEditorSubsystem->OpenEditorForAssets(ObjList, OpenedMethod);
		}
	}

	return bSuccessfulEditorOpen;
}

bool EditOrPreviewItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, EAssetTypeActivationMethod::Type ActivationMethod, EAssetTypeActivationOpenedMethod OpenedMethod)
{
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> EditOrPreviewPayloads;
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> ViewPayloads;

	ensure(ActivationMethod != EAssetTypeActivationMethod::Type::DoubleClicked);

	const bool bIsPreview = ActivationMethod == EAssetTypeActivationMethod::Type::Previewed;
	EnumerateAssetFileItemPayloads(InOwnerDataSource, InItems, [InAssetTools, bIsPreview, OpenedMethod, &EditOrPreviewPayloads, &ViewPayloads](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		if (bIsPreview)
		{
			if (CanPreviewAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
			{
				EditOrPreviewPayloads.Add(InAssetPayload);
			}
		}
		else if (OpenedMethod == EAssetTypeActivationOpenedMethod::Edit)
		{
			if (CanEditAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
			{
				EditOrPreviewPayloads.Add(InAssetPayload);
			}
			else if (CanViewAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
			{
				ViewPayloads.Add(InAssetPayload);
			}
		}
		else if (OpenedMethod == EAssetTypeActivationOpenedMethod::View)
		{
			if (CanViewAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
			{
				ViewPayloads.Add(InAssetPayload);
			}
		}
		return true;
	});

	const bool bEditItems = EditOrPreviewAssetFileItems(EditOrPreviewPayloads, ActivationMethod, EAssetTypeActivationOpenedMethod::Edit);
	const bool bViewItems = EditOrPreviewAssetFileItems(ViewPayloads, ActivationMethod, EAssetTypeActivationOpenedMethod::View);
	return bEditItems || bViewItems;
}

bool EditItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	return EditOrPreviewItems(InAssetTools, InOwnerDataSource, InItems, EAssetTypeActivationMethod::Type::Opened, EAssetTypeActivationOpenedMethod::Edit);
}

bool EditAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads)
{
	return EditOrPreviewAssetFileItems(InAssetPayloads, EAssetTypeActivationMethod::Type::Opened, EAssetTypeActivationOpenedMethod::Edit);
}

bool ViewItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	return EditOrPreviewItems(InAssetTools, InOwnerDataSource, InItems, EAssetTypeActivationMethod::Type::Opened, EAssetTypeActivationOpenedMethod::View);
}

bool ViewAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads)
{
	return EditOrPreviewAssetFileItems(InAssetPayloads, EAssetTypeActivationMethod::Type::Opened, EAssetTypeActivationOpenedMethod::View);
}

bool PreviewItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	return EditOrPreviewItems(InAssetTools, InOwnerDataSource, InItems, EAssetTypeActivationMethod::Type::Previewed, EAssetTypeActivationOpenedMethod::Edit);
}

bool PreviewAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads)
{
	return EditOrPreviewAssetFileItems(InAssetPayloads, EAssetTypeActivationMethod::Type::Previewed, EAssetTypeActivationOpenedMethod::Edit);
}

bool CanDuplicateItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanDuplicateAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanDuplicateAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	if (InAssetPayload.GetAssetData().IsRedirector())
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotDuplicateRedirectors", "Cannot duplicate redirectors"));
		return false;
	}

	if (const UClass* AssetClass = InAssetPayload.GetAssetData().GetClass())
	{
		if (AssetClass->IsChildOf<UClass>())
		{
			SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotDuplicateGeneratedClasses", "Cannot duplicate generated classes"));
			return false;
		}
	}

	if (TSharedPtr<IAssetTypeActions> AssetTypeActions = InAssetPayload.GetAssetTypeActions())
	{
		if (!AssetTypeActions->CanDuplicate(InAssetPayload.GetAssetData(), OutErrorMsg))
		{
			return false;
		}
	}

	return true;
}

bool DuplicateItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, UObject*& OutSourceAsset, FAssetData& OutNewAsset)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		if (CanDuplicateAssetFileItem(InAssetTools, *AssetPayload, nullptr))
		{
			return DuplicateAssetFileItem(InAssetTools, *AssetPayload, OutSourceAsset, OutNewAsset);
		}
	}

	return false;
}

bool DuplicateAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, UObject*& OutSourceAsset, FAssetData& OutNewAsset)
{
	// We need to potentially load the asset in order to duplicate it
	if (UObject* Asset = InAssetPayload.LoadAsset({ ULevel::LoadAllExternalObjectsTag }))
	{
		// Find a unique default name for the duplicated asset
		FString DefaultAssetName;
		FString PackageNameToUse;
		InAssetTools->CreateUniqueAssetName(Asset->GetOutermost()->GetPathName(), FString(), PackageNameToUse, DefaultAssetName);

		OutSourceAsset = Asset;
		OutNewAsset = FAssetData(*PackageNameToUse, *FPackageName::GetLongPackagePath(PackageNameToUse), *DefaultAssetName, Asset->GetClass()->GetClassPathName());
		return true;
	}

	return false;
}

bool DuplicateItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TArray<FAssetData>& OutNewAssets)
{
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> AssetPayloads;

	EnumerateAssetFileItemPayloads(InOwnerDataSource, InItems, [InAssetTools, &AssetPayloads](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		if (CanDuplicateAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
		{
			AssetPayloads.Add(InAssetPayload);
		}
		return true;
	});

	return DuplicateAssetFileItems(AssetPayloads, OutNewAssets);
}

bool DuplicateAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, TArray<FAssetData>& OutNewAssets)
{
	TArray<UObject*> ObjectsToDuplicate;

	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : InAssetPayloads)
	{
		// We need to potentially load the asset in order to duplicate it
		if (UObject* Asset = AssetPayload->LoadAsset({ ULevel::LoadAllExternalObjectsTag }))
		{
			ObjectsToDuplicate.Add(Asset);
		}
	}

	if (ObjectsToDuplicate.Num() > 0)
	{
		TArray<UObject*> NewObjects;
		ObjectTools::DuplicateObjects(ObjectsToDuplicate, FString(), FString(), /*bOpenDialog=*/false, &NewObjects);

		if (NewObjects.Num() > 0)
		{
			for (UObject* NewAsset : NewObjects)
			{
				OutNewAssets.Emplace(FAssetData(NewAsset));
			}

			return true;
		}
	}

	return false;
}

bool CanSaveItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanSaveAssetFileItem(InAssetTools, *AssetPayload, InSaveFlags, OutErrorMsg);
	}

	return false;
}

bool CanSaveAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg)
{
	if (!CanModifyAssetFileItem(InAssetTools, InAssetPayload, OutErrorMsg))
	{
		return false;
	}

	if (EnumHasAnyFlags(InSaveFlags, EContentBrowserItemSaveFlags::SaveOnlyIfLoaded | EContentBrowserItemSaveFlags::SaveOnlyIfDirty))
	{
		// Can't save a package that hasn't been loaded
		UPackage* Package = InAssetPayload.GetPackage(/*bTryRecacheIfNull*/true);
		if (!Package)
		{
			ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotSaveUnloadedAsset", "Cannot save unloaded item"));
			return false;
		}
		
		if (EnumHasAnyFlags(InSaveFlags, EContentBrowserItemSaveFlags::SaveOnlyIfDirty))
		{
			if (!Package->IsDirty())
			{
				ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotSaveNonDirtyAsset", "Cannot save an unmodified item"));
				return false;
			}
		}
	}

	return true;
}

bool SaveItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags)
{
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> AssetPayloads;

	EnumerateAssetFileItemPayloads(InOwnerDataSource, InItems, [InAssetTools, InSaveFlags, &AssetPayloads](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		if (CanSaveAssetFileItem(InAssetTools, *InAssetPayload, InSaveFlags, nullptr))
		{
			AssetPayloads.Add(InAssetPayload);
		}
		return true;
	});

	return SaveAssetFileItems(AssetPayloads, InSaveFlags);
}

bool SaveAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, const EContentBrowserItemSaveFlags InSaveFlags)
{
	TArray<UPackage*> PackagesToSave;

	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : InAssetPayloads)
	{
		if (UPackage* Package = EnumHasAnyFlags(InSaveFlags, EContentBrowserItemSaveFlags::SaveOnlyIfLoaded) ? AssetPayload->GetPackage() : AssetPayload->LoadPackage())
		{
			if (!EnumHasAnyFlags(InSaveFlags, EContentBrowserItemSaveFlags::SaveOnlyIfDirty) || Package->IsDirty())
			{
				PackagesToSave.Add(Package);
			}
		}
	}

	FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
	SaveParams.bCheckDirty = false;
	SaveParams.bPromptToSave = false;
	SaveParams.bIsExplicitSave = true;

	// TODO: Interactive vs non-interactive save?
	return PackagesToSave.Num() > 0
		&& FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, SaveParams) != FEditorFileUtils::PR_Failure;
}

bool IsRunningPIE(FText* OutErrorMsg)
{
	if (GIsEditor)
	{
		if (FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext())
		{
			SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotDeleteAssetInPIE", "Assets cannot be deleted while in PIE"));
			return true;
		}
	}
	return false;
}

bool CanDeleteItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return CanDeleteAssetFolderItem(InAssetTools, InAssetRegistry, *FolderPayload, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanDeleteAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanDeleteUnsupportedAssetFileItem(InAssetTools, *UnsupportedAssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanDeleteAssetFolderItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg)
{
	if (!CanModifyAssetFolderItem(InAssetTools, InFolderPayload, OutErrorMsg))
	{
		return false;
	}

	if (ContentBrowserDataUtils::IsTopLevelFolder(InFolderPayload.GetInternalPath()))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotDeleteRootFolders", "Cannot delete root folders"));
		return false;
	}

	if (IsRunningPIE(OutErrorMsg))
	{
		return false;
	}

	// Also check that sub-folders aren't locked, as this will be a recursive operation
	bool bCanModifyAllSubPaths = true;
	InAssetRegistry->EnumerateSubPaths(InFolderPayload.GetInternalPath(), [InAssetTools, OutErrorMsg, &bCanModifyAllSubPaths](FName InSubPath)
	{
		bCanModifyAllSubPaths &= CanModifyPath(InAssetTools, InSubPath, OutErrorMsg);
		return bCanModifyAllSubPaths;
	}, true);
	return bCanModifyAllSubPaths;
}

bool CanDeleteAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	if (!CanModifyAssetFileItem(InAssetTools, InAssetPayload, OutErrorMsg))
	{
		return false;
	}

	if (IsRunningPIE(OutErrorMsg))
	{
		return false;
	}

	if (InAssetPayload.GetAssetData().IsRedirector())
	{
		SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotDeleteRedirectors", "Cannot delete redirectors"));
		return false;
	}

	return true;
}

bool CanDeleteUnsupportedAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FText* OutErrorMsg)
{
	const FAssetData* AssetData = InUnsupportedAssetPayload.GetAssetDataIfAvailable();

	if (!AssetData)
	{
		// will need to investigate this latter
		return false;
	}

	if (!CanModifyPath(InAssetTools, AssetData->PackageName, OutErrorMsg))
	{
		return false;
	}

	if (IsRunningPIE(OutErrorMsg))
	{
		return false;
	}

	return true;
}

bool CanPrivatizeItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return false;
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanPrivatizeAssetFileItem(InAssetTools, *AssetPayload, OutErrorMsg);
	}

	return false;
}

bool CanPrivatizeAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg)
{
	if (!CanModifyAssetFileItem(InAssetTools, InAssetPayload, OutErrorMsg))
	{
		return false;
	}

	if (IsRunningPIE(OutErrorMsg))
	{
		return false;
	}

	if (InAssetPayload.GetAssetData().IsRedirector())
	{
		SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotPrivatizeRedirectors", "Cannot make redirectors private"));
		return false;
	}

	return true;
}

bool DeleteItems(FContentBrowserModule* InContentBrowserModule, IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	TArray<TSharedRef<const FContentBrowserAssetFolderItemDataPayload>, TInlineAllocator<16>> FolderPayloads;
	TArray<FAssetData> AssetDatas;
	AssetDatas.Reserve(16);

	auto ProcessAssetFolderItem = [InAssetTools, InAssetRegistry, &FolderPayloads](const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& InFolderPayload)
	{
		if (CanDeleteAssetFolderItem(InAssetTools, InAssetRegistry, *InFolderPayload, nullptr))
		{
			FolderPayloads.Add(InFolderPayload);
		}
		return true;
	};

	auto ProcessAssetFileItem = [InAssetTools, &AssetDatas](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		if (CanDeleteAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
		{
			AssetDatas.Add(InAssetPayload->GetAssetData());
		}
		return true;
	};

	auto ProcessUnsupportedAssetFileItem = [InAssetTools, &AssetDatas](const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>& InUnsupportedAssetPayload)
	{
		if (CanDeleteUnsupportedAssetFileItem(InAssetTools, *InUnsupportedAssetPayload, nullptr))
		{
			if (const FAssetData* AssetData = InUnsupportedAssetPayload->GetAssetDataIfAvailable())
			{
				AssetDatas.Add(*AssetData);
			}
		}
		return true;
	};

	EnumerateAssetItemPayloads(InOwnerDataSource, InItems, ProcessAssetFolderItem, ProcessAssetFileItem, ProcessUnsupportedAssetFileItem);

	bool bDidDelete = false;

	if (FolderPayloads.Num() > 0)
	{
		bDidDelete |= DeleteAssetFolderItems(InContentBrowserModule, FolderPayloads);
	}

	if (AssetDatas.Num() > 0)
	{
		bDidDelete |= ObjectTools::DeleteAssets(AssetDatas) > 0;
	}

	return bDidDelete;
}

bool PrivatizeItems(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> AssetPayloads;

	auto ProcessAssetFolderItem = [](const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& InFolderPayload)
	{
		return true;
	};

	auto ProcessAssetFileItem = [InAssetTools, &AssetPayloads](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		if (CanPrivatizeAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
		{
			AssetPayloads.Add(InAssetPayload);
		}
		return true;
	};

	EnumerateAssetItemPayloads(InOwnerDataSource, InItems, ProcessAssetFolderItem, ProcessAssetFileItem);

	if (AssetPayloads.IsEmpty())
	{
		return false;
	}

	TArray<FAssetData> AssetsToPrivatize;

	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : AssetPayloads)
	{
		AssetsToPrivatize.Add(AssetPayload->GetAssetData());
	}

	return ObjectTools::PrivatizeAssets(AssetsToPrivatize, /*bShowConfirmation*/true, InAssetAccessSpecifier) > 0;
}

bool DeleteAssetFolderItems(FContentBrowserModule* InContentBrowserModule, TArrayView<const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>> InFolderPayloads)
{
	TArray<FContentBrowserItemPath> ItemPathsToDelete;
	ItemPathsToDelete.Reserve(InFolderPayloads.Num());

	for (const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& FolderPayload : InFolderPayloads)
	{
		ItemPathsToDelete.Add(FContentBrowserItemPath(FolderPayload->GetInternalPath(), EContentBrowserPathType::Internal));
	}

	InContentBrowserModule->GetOnDeleteFolders().Broadcast(ItemPathsToDelete);

	TArray<FString> FoldersToDelete;
	Algo::Transform(ItemPathsToDelete, FoldersToDelete, [](FContentBrowserItemPath ItemPath) { return ItemPath.GetInternalPathString(); });

	return FoldersToDelete.Num() > 0
		&& AssetViewUtils::DeleteFolders(FoldersToDelete);
}

bool DeleteAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads)
{
	TArray<FAssetData> AssetsToDelete;

	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : InAssetPayloads)
	{
		AssetsToDelete.Add(AssetPayload->GetAssetData());
	}

	return AssetsToDelete.Num() > 0
		&& ObjectTools::DeleteAssets(AssetsToDelete) > 0;
}

bool DeleteUnsupportedAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>> InUnsupportedAssetPayloads)
{
	TArray<FAssetData> AssetsToDelete;

	for (const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>& UnsupportedAssetPayload : InUnsupportedAssetPayloads)
	{
		if (const FAssetData* AssetData = UnsupportedAssetPayload->GetAssetDataIfAvailable())
		{
			AssetsToDelete.Add(*AssetData);
		}
	}

	return AssetsToDelete.Num() > 0
		&& ObjectTools::DeleteAssets(AssetsToDelete) > 0;
}

bool CanRenameItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString* InNewName, FText* OutErrorMsg)
{
	constexpr bool bCheckUniqueName = true;
	return CanRenameItem(InAssetTools, InOwnerDataSource, InItem, bCheckUniqueName, InNewName, OutErrorMsg);
}

bool CanRenameItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return CanRenameAssetFolderItem(InAssetTools, *FolderPayload, bCheckUniqueName, InNewName, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanRenameAssetFileItem(InAssetTools, *AssetPayload, bCheckUniqueName, InNewName, InItem.IsTemporary(), OutErrorMsg);
	}

	return false;
}

bool CanRenameAssetFolderItem(IAssetTools* InAssetTools, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const FString* InNewName, FText* OutErrorMsg)
{
	constexpr bool bCheckUniqueName = true;
	return CanRenameAssetFolderItem(InAssetTools, InFolderPayload, bCheckUniqueName, InNewName, OutErrorMsg);
}

bool CanRenameAssetFolderItem(IAssetTools* InAssetTools, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg)
{
	if (!CanModifyAssetFolderItem(InAssetTools, InFolderPayload, OutErrorMsg))
	{
		return false;
	}

	if (ContentBrowserDataUtils::IsTopLevelFolder(InFolderPayload.GetInternalPath()))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotRenameRootFolders", "Cannot rename root folders"));
		return false;
	}

	if (InNewName)
	{
		const FString FolderPath = FPaths::GetPath(InFolderPayload.GetInternalPath().ToString());

		FText ValidationErrorMsg;
		const bool bAllowExistingFolder = !bCheckUniqueName;
		if (!AssetViewUtils::IsValidFolderPathForCreate(FolderPath, *InNewName, ValidationErrorMsg, bAllowExistingFolder))
		{
			ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, ValidationErrorMsg);
			return false;
		}
	}

	return true;
}

bool CanRenameAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const FString* InNewName, const bool InIsTempoarary, FText* OutErrorMsg)
{
	constexpr bool bCheckUniqueName = true;
	return CanRenameAssetFileItem(InAssetTools, InAssetPayload, bCheckUniqueName, InNewName, InIsTempoarary, OutErrorMsg);
}

bool CanRenameAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const bool bCheckUniqueName, const FString* InNewName, const bool InIsTempoarary, FText* OutErrorMsg)
{
	if (!CanModifyAssetFileItem(InAssetTools, InAssetPayload, OutErrorMsg))
	{
		return false;
	}

	if (InAssetPayload.GetAssetData().IsRedirector())
	{
		SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_CannotRenameRedirectors", "Cannot rename redirectors"));
		return false;
	}

	if (InNewName && (InIsTempoarary || InNewName->Len() >= NAME_SIZE || InAssetPayload.GetAssetData().AssetName != FName(**InNewName))) // Deliberately ignore case here to allow case-only renames of existing assets
	{
		const FString PackageName = InAssetPayload.GetAssetData().PackagePath.ToString() / *InNewName;
		const FString ObjectPath = FString::Printf(TEXT("%s.%s"), *PackageName, **InNewName);

		FText ValidationErrorMsg;
		const bool bAllowExistingAsset = !bCheckUniqueName;
		if (!AssetViewUtils::IsValidObjectPathForCreate(ObjectPath, ValidationErrorMsg, bAllowExistingAsset))
		{
			SetOptionalErrorMessage(OutErrorMsg, ValidationErrorMsg);
			return false;
		}
	}

	if (TSharedPtr<IAssetTypeActions> AssetTypeActions = InAssetPayload.GetAssetTypeActions())
	{
		if (!AssetTypeActions->CanRename(InAssetPayload.GetAssetData(), OutErrorMsg))
		{
			return false;
		}
	}

	return true;
}

bool RenameItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString& InNewName)
{
	constexpr bool bCheckUniqueName = false;

	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		if (CanRenameAssetFolderItem(InAssetTools, *FolderPayload, bCheckUniqueName, &InNewName, nullptr))
		{
			return RenameAssetFolderItem(InAssetRegistry, *FolderPayload, InNewName);
		}
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		if (CanRenameAssetFileItem(InAssetTools, *AssetPayload, bCheckUniqueName, &InNewName, InItem.IsTemporary(), nullptr))
		{
			return RenameAssetFileItem(InAssetTools, *AssetPayload, InNewName);
		}
	}

	return false;
}

bool RenameAssetFolderItem(IAssetRegistry* InAssetRegistry, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const FString& InNewName)
{
	const FString OldPath = InFolderPayload.GetInternalPath().ToString();
	const FString NewPath = FPaths::GetPath(OldPath) / InNewName;

	// Ensure the folder exists on disk
	FString NewPathOnDisk;
	if (FPackageName::TryConvertLongPackageNameToFilename(NewPath, NewPathOnDisk) && IFileManager::Get().MakeDirectory(*NewPathOnDisk, true))
	{
		InAssetRegistry->AddPath(NewPath);
		if (AssetViewUtils::RenameFolder(NewPath, OldPath))
		{
			return true;
		}
	}

	return false;
}

bool RenameAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const FString& InNewName)
{
	// We need to potentially load the asset in order to rename it
	if (UObject* Asset = InAssetPayload.LoadAsset({ ULevel::LoadAllExternalObjectsTag }))
	{
		TArray<UObject*> AssetsToRename = {Asset};
		FOnPreDestructiveAssetActionArgs PreDestructiveArgs(AssetsToRename, EDestructiveAssetActions::AssetRename);
		FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);
		if (!PreDestructiveArgs.Result.bSuccess)
		{
			UE_LOGF(LogContentBrowserAssetDataSource, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
			return false;
		}

		const FString PackagePath = FPackageName::GetLongPackagePath(Asset->GetOutermost()->GetName());

		TArray<FAssetRenameData> AssetsAndNames;
		const bool bSoftReferenceOnly = false;
		const bool bAlsoRenameLocalizedVariants = true;
		AssetsAndNames.Emplace(FAssetRenameData(Asset, PackagePath, InNewName, bSoftReferenceOnly, bAlsoRenameLocalizedVariants));
		// Note: This also returns false for Pending results as the rename may yet fail or be canceled, so the change has to be detected later via the asset registry
		return InAssetTools->RenameAssetsWithDialog(AssetsAndNames) == EAssetRenameResult::Success;
	}

	return false;
}

bool CopyItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	// The destination path must be writable
	if (!CanModifyPath(InAssetTools, InDestPath, nullptr))
	{
		return false;
	}

	TArray<TSharedRef<const FContentBrowserAssetFolderItemDataPayload>, TInlineAllocator<16>> FolderPayloads;
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> AssetPayloads;

	auto ProcessAssetFolderItem = [&FolderPayloads](const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& InFolderPayload)
	{
		FolderPayloads.Add(InFolderPayload);
		return true;
	};

	auto ProcessAssetFileItem = [&AssetPayloads](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		AssetPayloads.Add(InAssetPayload);
		return true;
	};

	EnumerateAssetItemPayloads(InOwnerDataSource, InItems, ProcessAssetFolderItem, ProcessAssetFileItem);

	bool bDidCopy = false;

	if (FolderPayloads.Num() > 0)
	{
		bDidCopy |= CopyAssetFolderItems(FolderPayloads, InDestPath);
	}

	if (AssetPayloads.Num() > 0)
	{
		bDidCopy |= CopyAssetFileItems(AssetPayloads, InDestPath);
	}

	return bDidCopy;
}

bool CopyAssetFolderItems(TArrayView<const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>> InFolderPayloads, const FName InDestPath)
{
	TArray<FString> FoldersToCopy;

	for (const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& FolderPayload : InFolderPayloads)
	{
		FoldersToCopy.Add(FolderPayload->GetInternalPath().ToString());
	}

	return FoldersToCopy.Num() > 0
		&& AssetViewUtils::CopyFolders(FoldersToCopy, InDestPath.ToString());
}

bool CopyAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, const FName InDestPath)
{
	TArray<UObject*> AssetsToCopy;

	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : InAssetPayloads)
	{
		// We need to potentially load the asset in order to duplicate it
		if (UObject* Asset = AssetPayload->LoadAsset({ ULevel::LoadAllExternalObjectsTag }))
		{
			AssetsToCopy.Add(Asset);
		}
	}

	if (AssetsToCopy.Num() > 0)
	{
		TArray<UObject*> NewObjects;
		ObjectTools::DuplicateObjects(AssetsToCopy, FString(), InDestPath.ToString(), /*bOpenDialog=*/false, &NewObjects);

		return NewObjects.Num() > 0;
	}

	return false;
}

bool MoveItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	// The destination path must be writable
	if (!CanModifyPath(InAssetTools, InDestPath, nullptr))
	{
		return false;
	}

	TArray<TSharedRef<const FContentBrowserAssetFolderItemDataPayload>, TInlineAllocator<16>> FolderPayloads;
	TArray<TSharedRef<const FContentBrowserAssetFileItemDataPayload>, TInlineAllocator<16>> AssetPayloads;

	auto ProcessAssetFolderItem = [InAssetTools, &FolderPayloads](const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& InFolderPayload)
	{
		// Moving has to be able to delete the original item
		if (CanModifyAssetFolderItem(InAssetTools, *InFolderPayload, nullptr))
		{
			FolderPayloads.Add(InFolderPayload);
		}
		return true;
	};

	auto ProcessAssetFileItem = [InAssetTools, &AssetPayloads](const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& InAssetPayload)
	{
		// Moving has to be able to delete the original item
		if (CanModifyAssetFileItem(InAssetTools, *InAssetPayload, nullptr))
		{
			AssetPayloads.Add(InAssetPayload);
		}
		return true;
	};

	EnumerateAssetItemPayloads(InOwnerDataSource, InItems, ProcessAssetFolderItem, ProcessAssetFileItem);

	bool bDidMove = false;

	if (FolderPayloads.Num() > 0)
	{
		bDidMove |= MoveAssetFolderItems(FolderPayloads, InDestPath);
	}

	if (AssetPayloads.Num() > 0)
	{
		bDidMove |= MoveAssetFileItems(AssetPayloads, InDestPath);
	}

	return bDidMove;
}

bool MoveAssetFolderItems(TArrayView<const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>> InFolderPayloads, const FName InDestPath)
{
	TArray<FString> FoldersToMove;

	for (const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>& FolderPayload : InFolderPayloads)
	{
		FoldersToMove.Add(FolderPayload->GetInternalPath().ToString());
	}

	return FoldersToMove.Num() > 0
		&& AssetViewUtils::MoveFolders(FoldersToMove, InDestPath.ToString());
}

bool MoveAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, const FName InDestPath)
{
	TArray<UObject*> AssetsToMove;
	AssetsToMove.Reserve(InAssetPayloads.Num());

	TArray<FTopLevelAssetPath> Destinations;
	Destinations.Reserve(InAssetPayloads.Num());

	const FString DestPath = InDestPath.ToString();
	for (const TSharedRef<const FContentBrowserAssetFileItemDataPayload>& AssetPayload : InAssetPayloads)
	{
		// We need to potentially load the asset in order to duplicate it
		if (UObject* Asset = AssetPayload->LoadAsset({ ULevel::LoadAllExternalObjectsTag }))
		{
			AssetsToMove.Add(Asset);

			FSoftObjectPath SourceAssetPath(Asset);
			FNameBuilder SourcePackagePath(SourceAssetPath.GetLongPackageFName());
			FNameBuilder DestPackagePath;
			DestPackagePath.Append(DestPath);
			FPathViews::AppendPath(DestPackagePath, FPathViews::GetBaseFilename(SourcePackagePath.ToView()));
			FTopLevelAssetPath DestTopLevelAssetPath(FName(DestPackagePath.ToView()), SourceAssetPath.GetAssetFName());
			Destinations.Add(DestTopLevelAssetPath);
		}
	}

	FOnPreDestructiveAssetActionArgs PreDestructiveArgs(AssetsToMove, EDestructiveAssetActions::AssetMove, Destinations);
	FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);
	if (!PreDestructiveArgs.Result.bSuccess)
	{
		UE_LOGF(LogContentBrowserAssetDataSource, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
		return false;
	}

	if (AssetsToMove.Num() > 0)
	{
		AssetViewUtils::MoveAssets(AssetsToMove, DestPath);
		return true;
	}

	return false;
}

bool IsItemDirty(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return IsAssetFileItemDirty(*AssetPayload);
	}

	return false;
}

bool IsAssetFileItemDirty(const FContentBrowserAssetFileItemDataPayload& InAssetPayload)
{
	if (const UPackage* AssetPackage = InAssetPayload.GetPackage())
	{
		if (const UAssetDefinition* AssetDefinition = InAssetPayload.GetAssetDefinition())
		{
			if (AssetDefinition->ShouldSaveExternalPackages())
			{
				for (const UPackage* ExternalPackage : AssetPackage->GetExternalPackages())
				{
					if (ExternalPackage && ExternalPackage->IsDirty())
					{
						return true;
					}
				}
			}
		}

		return AssetPackage->IsDirty();
	}
	
	return false;
}

bool UpdateItemThumbnail(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return UpdateAssetFileItemThumbnail(*AssetPayload, InThumbnail);
	}

	return false;
}

bool UpdateAssetFileItemThumbnail(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FAssetThumbnail& InThumbnail)
{
	InAssetPayload.UpdateThumbnail(InThumbnail);
	return true;
}

bool AppendItemReference(IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return AppendAssetFolderItemReference(InAssetRegistry, *FolderPayload, InOutStr);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendAssetFileItemReference(*AssetPayload, InOutStr);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendUnsupportedAssetFileItemReference(*UnsupportedAssetPayload, InOutStr);
	}

	return false;
}

bool AppendItemObjectPath(IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return AppendAssetFolderItemReference(InAssetRegistry, *FolderPayload, InOutStr);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendAssetFileObjectPath(*AssetPayload, InOutStr);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendUnsupportedAssetFileObjectPath(*UnsupportedAssetPayload, InOutStr);
	}

	return false;
}

bool AppendItemPackageName(IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return AppendAssetFolderItemReference(InAssetRegistry, *FolderPayload, InOutStr);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendAssetFilePackageName(*AssetPayload, InOutStr);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendUnsupportedAssetFilePackageName(*UnsupportedAssetPayload, InOutStr);
	}

	return false;
}

void AppendAssetExportText(const FAssetData& AssetData, FString& InOutStr)
{
	if (InOutStr.IsEmpty())
	{
		AssetData.GetExportTextName(InOutStr);
	}
	else
	{
		if (InOutStr.Len() > 0)
		{
			InOutStr += LINE_TERMINATOR;
		}
		InOutStr += AssetData.GetExportTextName();
	}
}

void AppendAssetObjectPath(const FAssetData& AssetData, FString& InOutStr)
{
	if (!InOutStr.IsEmpty())
	{
		InOutStr += LINE_TERMINATOR;
	}
	AssetData.AppendObjectPath(InOutStr);
}

void AppendAssetPackageName(const FAssetData& AssetData, FString& InOutStr)
{
	if (!InOutStr.IsEmpty())
	{
		InOutStr += LINE_TERMINATOR;
	}
	AssetData.PackageName.AppendString(InOutStr);
}

bool AppendAssetFolderItemReference(IAssetRegistry* InAssetRegistry, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FString& InOutStr)
{
	// Folders gather the assets references from within them
	FARFilter AssetFilter;
	AssetFilter.PackagePaths.Add(InFolderPayload.GetInternalPath());
	InAssetRegistry->EnumerateAssets(AssetFilter, [&InOutStr](const FAssetData& AssetData)
	{
		if (ContentBrowserAssetData::IsPrimaryAsset(AssetData))
		{
			AppendAssetExportText(AssetData, InOutStr);
		}
		return true;
	});

	return true;
}

bool AppendAssetFileItemReference(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& InOutStr)
{
	AppendAssetExportText(InAssetPayload.GetAssetData(), InOutStr);
	return true;
}

bool AppendAssetFileObjectPath(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& InOutStr)
{
	AppendAssetObjectPath(InAssetPayload.GetAssetData(), InOutStr);
	return true;
}

bool AppendAssetFilePackageName(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& InOutStr)
{
	AppendAssetPackageName(InAssetPayload.GetAssetData(), InOutStr);
	return true;
}

bool AppendUnsupportedAssetFileItemReference(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& InOutStr)
{
	if (const FAssetData* AssetData = InUnsupportedAssetPayload.GetAssetDataIfAvailable())
	{
		AppendAssetExportText(*AssetData, InOutStr);
		return true;
	}

	return false;
}

bool AppendUnsupportedAssetFileObjectPath(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& InOutStr)
{
	if (const FAssetData* AssetData = InUnsupportedAssetPayload.GetAssetDataIfAvailable())
	{
		AppendAssetObjectPath(*AssetData, InOutStr);
		return true;
	}

	return false;
}

bool AppendUnsupportedAssetFilePackageName(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& InOutStr)
{
	if (const FAssetData* AssetData = InUnsupportedAssetPayload.GetAssetDataIfAvailable())
	{
		AppendAssetPackageName(*AssetData, InOutStr);
		return true;
	}

	return false;
}

bool GetItemPhysicalPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetAssetFolderItemPhysicalPath(*FolderPayload, OutDiskPath);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetAssetFileItemPhysicalPath(*AssetPayload, OutDiskPath);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetUnsupportedAssetFileItemPhysicalPath(*UnsupportedAssetPayload, OutDiskPath);
	}

	return false;
}

bool GetAssetFolderItemPhysicalPath(const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FString& OutDiskPath)
{
	const FString& FolderFilename = InFolderPayload.GetFilename();
	if (!FolderFilename.IsEmpty())
	{
		OutDiskPath = FolderFilename;
		return true;
	}

	return false;
}

bool GetAssetFileItemPhysicalPath(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& OutDiskPath)
{
	const FString& AssetFilename = InAssetPayload.GetFilename();
	if (!AssetFilename.IsEmpty())
	{
		OutDiskPath = AssetFilename;
		return true;
	}

	return false;
}

bool GetUnsupportedAssetFileItemPhysicalPath(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& OutDiskPath)
{
	const FString& AssetFilename = InUnsupportedAssetPayload.GetFilename();
	if (!AssetFilename.IsEmpty())
	{
		OutDiskPath = AssetFilename;
		return true;
	}

	return false;
}

void GetClassItemAttribute(const FAssetData& InAssetData, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	check(InAssetData.IsValid());

	OutAttributeValue.SetValue(InAssetData.AssetClassPath.ToString());

	if (InIncludeMetaData)
	{
		static const FText ClassDisplayName = LOCTEXT("AttributeDisplayName_Class", "Class");

		FContentBrowserItemDataAttributeMetaData AttributeMetaData;
		AttributeMetaData.AttributeType = UObject::FAssetRegistryTag::TT_Hidden;
		AttributeMetaData.DisplayName = ClassDisplayName;
		OutAttributeValue.SetMetaData(MoveTemp(AttributeMetaData));
	}
}

bool GetDiskSizeItemAttribute(const FAssetData& InAssetData, IAssetRegistry* InAssetRegistry, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	check(InAssetData.IsValid());
	check(InAssetRegistry);

	if (TOptional<FAssetPackageData> PackageData = InAssetRegistry->GetAssetPackageDataCopy(InAssetData.PackageName))
	{
		OutAttributeValue.SetValue(PackageData->DiskSize);

		if (InIncludeMetaData)
		{
			static const FText DiskSizeDisplayName = LOCTEXT("AttributeDisplayName_DiskSize", "Disk Size");

			FContentBrowserItemDataAttributeMetaData AttributeMetaData;
			AttributeMetaData.AttributeType = UObject::FAssetRegistryTag::TT_Numerical;
			AttributeMetaData.DisplayFlags = UObject::FAssetRegistryTag::TD_Memory;
			AttributeMetaData.DisplayName = DiskSizeDisplayName;
			OutAttributeValue.SetMetaData(MoveTemp(AttributeMetaData));
		}
		return true;
	}
	return false;
}

bool GetVirtualizationItemAttribute(const FAssetData& InAssetData, IAssetRegistry* InAssetRegistry, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	check(InAssetData.IsValid());
	check(InAssetRegistry != nullptr);

	if (TOptional<FAssetPackageData> PackageData = InAssetRegistry->GetAssetPackageDataCopy(InAssetData.PackageName))
	{
		// Note that we do not localize True/False as nowhere else in the code base seems to do this
		// and it would be odd for only a single entry in the tool tip to localize booleans.

		if (UE::Virtualization::IVirtualizationSystem::Get().IsEnabled())
		{
			if (PackageData->FileVersionUE >= EUnrealEngineObjectUE5Version::PAYLOAD_TOC)
			{
				OutAttributeValue.SetValue(PackageData->HasVirtualizedPayloads() ? TEXT("True") : TEXT("False"));
			}
			else
			{
				OutAttributeValue.SetValue(LOCTEXT("AttributeDisplayName_OutOfDate", "Version too old, resave to enable"));
			}
		}
		else
		{
			if (PackageData->HasVirtualizedPayloads())
			{
				OutAttributeValue.SetValue(TEXT("True"));
			}
			else
			{
				// The VA system is disabled and nothing in the package is virtualized, so no need to display this in the tool tip
				return false;
			}
		}

		if (InIncludeMetaData)
		{
			static const FText DisplayName = LOCTEXT("AttributeDisplayName_VirtualizedData", "Has Virtualized Data");

			FContentBrowserItemDataAttributeMetaData AttributeMetaData;
			AttributeMetaData.AttributeType = UObject::FAssetRegistryTag::TT_Alphabetical;
			AttributeMetaData.DisplayFlags = UObject::FAssetRegistryTag::TD_None;
			AttributeMetaData.DisplayName = DisplayName;

			OutAttributeValue.SetMetaData(MoveTemp(AttributeMetaData));
		}

		return true;
	}
	else
	{
		return false;
	}
}

void GetGenericItemAttribute(const FName InTagKey, const FString& InTagValue, const FAssetPropertyTagCache::FClassPropertyTagCache* InClassPropertyTagCache, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	check(!InTagKey.IsNone());

	if (FTextStringHelper::IsComplexText(*InTagValue))
	{
		FText TmpText;
		if (FTextStringHelper::ReadFromBuffer(*InTagValue, TmpText))
		{
			OutAttributeValue.SetValue(TmpText);
		}
	}
	if (!OutAttributeValue.IsValid())
	{
		OutAttributeValue.SetValue(InTagValue);
	}

	if (InIncludeMetaData)
	{
		FContentBrowserItemDataAttributeMetaData AttributeMetaData;
		const FAssetPropertyTagCache::FPropertyTagCache* PropertyTagCache = InClassPropertyTagCache ? InClassPropertyTagCache->GetCacheForTag(InTagKey) : nullptr;
		if (PropertyTagCache)
		{
			AttributeMetaData.AttributeType = PropertyTagCache->TagType;
			AttributeMetaData.DisplayFlags = PropertyTagCache->DisplayFlags;
			AttributeMetaData.DisplayName = PropertyTagCache->DisplayName;
			AttributeMetaData.TooltipText = PropertyTagCache->TooltipText;
			AttributeMetaData.Suffix = PropertyTagCache->Suffix;
			AttributeMetaData.bIsImportant = !PropertyTagCache->ImportantValue.IsEmpty() && PropertyTagCache->ImportantValue == InTagValue;
		}
		else
		{
			AttributeMetaData.DisplayName = FText::AsCultureInvariant(FName::NameToDisplayString(InTagKey.ToString(), /*bIsBool*/false));
		}
		OutAttributeValue.SetMetaData(MoveTemp(AttributeMetaData));
	}
}

bool GetItemAttribute(FContentBrowserModule* InContentBrowserModule, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	// First see if another module hooked this to provide its own customizations. If it did, we can be done here.
	for(const FGetItemDataAttribute& Delegate : InContentBrowserModule->GetAllItemDataAttributeDelegates())
	{
		if (Delegate.Execute(InItem, InIncludeMetaData, InAttributeKey, OutAttributeValue))
		{
			return true;
		}
	}

	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetAssetFolderItemAttribute(*FolderPayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetAssetFileItemAttribute(*AssetPayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetUnsupportedAssetFileItemAttribute(*UnsupportedAssetPayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	if (InItem.IsFolder() && InItem.IsDisplayOnlyFolder())
	{
		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsCustomVirtualFolder)
		{
			// Exclude certain built-in folders from being visualized as "virtual" organizational folders
			// This is somewhat hacky and could be avoided by constructing the virtual folders up front
			// See UContentBrowserAssetDataSource::BuildRootPathVirtualTree and UContentBrowserDataSubsystem::ConvertInternalPathToVirtual
			static TSet<FName> ExcludeFoldersFromCustomIcon = []() {
				TSet<FName> Set;
				Set.Add("/");
				Set.Add("/All");
				Set.Add("/GameData");
				Set.Add("/All/GameData");
				Set.Add("/EngineData");
				Set.Add("/All/EngineData");
				Set.Add("/Plugins");
				Set.Add("/All/Plugins");
				return Set;
			}();

			bool bIsCustomVirtual = !ExcludeFoldersFromCustomIcon.Contains(InItem.GetVirtualPath());
			OutAttributeValue.SetValue<bool>(bIsCustomVirtual);
			return true;
		}
	}

	return false;
}

bool GetAssetFolderItemAttribute(const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	// Hard-coded attribute keys
	{
		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsDeveloperContent)
		{
			const bool bIsDevelopersFolder = AssetViewUtils::IsDevelopersFolder(InFolderPayload.GetInternalPath().ToString());
			OutAttributeValue.SetValue(bIsDevelopersFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsLocalizedContent)
		{
			const bool bIsLocalizedFolder = FPackageName::IsLocalizedPackage(InFolderPayload.GetInternalPath().ToString());
			OutAttributeValue.SetValue(bIsLocalizedFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsEngineContent)
		{
			const bool bIsEngineFolder = AssetViewUtils::IsEngineFolder(InFolderPayload.GetInternalPath().ToString(), /*bIncludePlugins*/true);
			OutAttributeValue.SetValue(bIsEngineFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsProjectContent)
		{
			const bool bIsProjectFolder = AssetViewUtils::IsProjectFolder(InFolderPayload.GetInternalPath().ToString(), /*bIncludePlugins*/true);
			OutAttributeValue.SetValue(bIsProjectFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsPluginContent)
		{
			const bool bIsPluginFolder = AssetViewUtils::IsPluginFolder(InFolderPayload.GetInternalPath().ToString());
			OutAttributeValue.SetValue(bIsPluginFolder);
			return true;
		}
	}

	return false;
}

bool GetAssetFileItemAttribute(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	if (GetAssetDataAttribute(InAssetPayload.GetAssetData(), InIncludeMetaData, InAttributeKey, OutAttributeValue))
	{
		return true;
	}

	// Hard-coded attribute keys
	{
		if (InAttributeKey == ContentBrowserItemAttributes::ItemTypeDisplayName)
		{
			if (TSharedPtr<IAssetTypeActions> AssetTypeActions = InAssetPayload.GetAssetTypeActions())
			{
				FText AssetDisplayName = AssetTypeActions->GetDisplayNameFromAssetData(InAssetPayload.GetAssetData());
				if (!AssetDisplayName.IsEmpty())
				{
					OutAttributeValue.SetValue(MoveTemp(AssetDisplayName));
				}
				else
				{
					OutAttributeValue.SetValue(AssetTypeActions->GetName());
				}
				return true;
			}
			return false;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemDescription)
		{
			if (const UAssetDefinition* AssetDefinition = InAssetPayload.GetAssetDefinition())
			{
				FWarnIfAssetsLoadedInScope WarnIfAssetLoadedInScope({ InAssetPayload.GetAssetData() });
				
				FText AssetDescription = AssetDefinition->GetAssetDescription(InAssetPayload.GetAssetData());
				if (!AssetDescription.IsEmpty())
				{
					OutAttributeValue.SetValue(MoveTemp(AssetDescription));
					return true;
				}
			}
			return false;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemColor)
		{
			if (TSharedPtr<IAssetTypeActions> AssetTypeActions = InAssetPayload.GetAssetTypeActions())
			{
				const FLinearColor AssetColor = AssetTypeActions->GetTypeColor();
				OutAttributeValue.SetValue(AssetColor.ToString());
				return true;
			}
			return false;
		}
	}


	return false;
}

bool GetUnsupportedAssetFileItemAttribute(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	if (const FAssetData* AssetData = InUnsupportedAssetPayload.GetAssetDataIfAvailable())
	{
		return GetAssetDataAttribute(*AssetData, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	return false;
}

bool GetAssetDataAttribute(const FAssetData& InAssetData, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	// Hard-coded attribute keys
	{
		static const FName NAME_Type = "Type";

		if (InAttributeKey == ContentBrowserItemAttributes::ItemTypeName || InAttributeKey == NAME_Class || InAttributeKey == NAME_Type)
		{
			GetClassItemAttribute(InAssetData, InIncludeMetaData, OutAttributeValue);
			return true;
		}


		if (InAttributeKey == ContentBrowserItemAttributes::ItemDiskSize)
		{
			return GetDiskSizeItemAttribute(InAssetData, IAssetRegistry::Get(), InIncludeMetaData, OutAttributeValue);
		}

		if (InAttributeKey == ContentBrowserItemAttributes::VirtualizedData)
		{
			return GetVirtualizationItemAttribute(InAssetData, IAssetRegistry::Get(), InIncludeMetaData, OutAttributeValue);
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsDeveloperContent)
		{
			const bool bIsDevelopersFolder = AssetViewUtils::IsDevelopersFolder(InAssetData.PackageName.ToString());
			OutAttributeValue.SetValue(bIsDevelopersFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsLocalizedContent)
		{
			const bool bIsLocalizedFolder = FPackageName::IsLocalizedPackage(InAssetData.PackageName.ToString());
			OutAttributeValue.SetValue(bIsLocalizedFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsEngineContent)
		{
			const bool bIsEngineFolder = AssetViewUtils::IsEngineFolder(InAssetData.PackageName.ToString(), /*bIncludePlugins*/true);
			OutAttributeValue.SetValue(bIsEngineFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsProjectContent)
		{
			const bool bIsProjectFolder = AssetViewUtils::IsProjectFolder(InAssetData.PackageName.ToString(), /*bIncludePlugins*/true);
			OutAttributeValue.SetValue(bIsProjectFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsPluginContent)
		{
			const bool bIsPluginFolder = AssetViewUtils::IsPluginFolder(InAssetData.PackageName.ToString());
			OutAttributeValue.SetValue(bIsPluginFolder);
			return true;
		}
	}

	// Generic attribute keys
	{
		const FAssetPropertyTagCache::FClassPropertyTagCache* ClassPropertyTagCache = FAssetPropertyTagCache::Get().FindCacheForClass(InAssetData.AssetClassPath);

		FName FoundAttributeKey = InAttributeKey;
		FAssetDataTagMapSharedView::FFindTagResult FoundValue = InAssetData.TagsAndValues.FindTag(FoundAttributeKey);
		if (!FoundValue.IsSet())
		{
			//ensureMsgf(ClassPropertyTagCache || (FindObject<UClass>(InAssetData.AssetClassPath) == nullptr), TEXT("FAssetPropertyTagCache not populated for type %s when looking for attribute %s"), *WriteToString<256>(InAssetData.AssetClassPath), *WriteToString<256>(InAttributeKey));
			if (ClassPropertyTagCache)
			{
				// Check to see if the key we were given resolves as an alias
				FoundAttributeKey = ClassPropertyTagCache->GetTagNameFromAlias(FoundAttributeKey);
				if (!FoundAttributeKey.IsNone())
				{
					FoundValue = InAssetData.TagsAndValues.FindTag(FoundAttributeKey);
				}
			}
		}
		if (FoundValue.IsSet())
		{
			GetGenericItemAttribute(FoundAttributeKey, FoundValue.GetValue(), ClassPropertyTagCache, InIncludeMetaData, OutAttributeValue);
			return true;
		}
	}

	return false;
}

bool GetItemAttributes(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetAssetFileItemAttributes(*AssetPayload, InIncludeMetaData, OutAttributeValues);
	}

	if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetUnsupportedAssetFileItemAttributes(*UnsupportedAssetPayload, InIncludeMetaData, OutAttributeValues);
	}

	return false;
}

bool GetAssetFileItemAttributes(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	return GetAssetDataAttributes(InAssetPayload.GetAssetData(), InIncludeMetaData, OutAttributeValues);
}

bool GetUnsupportedAssetFileItemAttributes(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	if (const FAssetData* AssetData = InUnsupportedAssetPayload.GetAssetDataIfAvailable())
	{
		return GetAssetDataAttributes(*AssetData, InIncludeMetaData, OutAttributeValues);
	}

	return false;
}

bool GetAssetDataAttributes(const FAssetData& InAssetData, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	// Hard-coded attribute keys
	{
		// Class
		FContentBrowserItemDataAttributeValue& ClassAttributeValue = OutAttributeValues.Add(NAME_Class);
		GetClassItemAttribute(InAssetData, InIncludeMetaData, ClassAttributeValue);

		// Disk Size
		FContentBrowserItemDataAttributeValue DiskSizeAttributeValue;
		if (GetDiskSizeItemAttribute(InAssetData, IAssetRegistry::Get(), InIncludeMetaData, DiskSizeAttributeValue))
		{
			OutAttributeValues.Add(ContentBrowserItemAttributes::ItemDiskSize, MoveTemp(DiskSizeAttributeValue));
		}

		// Virtualized Payloads
		FContentBrowserItemDataAttributeValue VirtualizedAttributeValue;
		if (GetVirtualizationItemAttribute(InAssetData, IAssetRegistry::Get(), InIncludeMetaData, VirtualizedAttributeValue))
		{
			OutAttributeValues.Add(ContentBrowserItemAttributes::VirtualizedData, MoveTemp(VirtualizedAttributeValue));
		}
	}

	// Generic attribute keys
	static const FTopLevelAssetPath BlueprintAssetClass = FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Blueprint"));
	static const FName ParentClassTag = FName("ParentClass");
	{
		const FAssetPropertyTagCache::FClassPropertyTagCache* ClassPropertyTagCache = FAssetPropertyTagCache::Get().FindCacheForClass(InAssetData.AssetClassPath);
		const FAssetPropertyTagCache::FClassPropertyTagCache* ParentClassPropertyTagCache = nullptr;

		if (InAssetData.AssetClassPath == BlueprintAssetClass)
		{
			FAssetTagValueRef ParentClassRef = InAssetData.TagsAndValues.FindTag(ParentClassTag);
			if (ParentClassRef.IsSet())
			{
				FString ParentClassName(ParentClassRef.AsString());
				FTopLevelAssetPath ParentClassPathName = UClass::TryConvertShortTypeNameToPathName<UClass>(ParentClassName, ELogVerbosity::Warning, TEXT("GetAssetFileItemAttributes"));
				if (!ParentClassPathName.IsNull())
				{
					ParentClassPropertyTagCache = FAssetPropertyTagCache::Get().FindCacheForClass(ParentClassPathName);
				}
				else
				{
					UE_LOGF(LogContentBrowserAssetDataSource, Warning, "Unable to convert short ParentClass name \"%ls\" to path name", *ParentClassName);
				}
			}
		}

		/*ensureMsgf(!InIncludeMetaData || ClassPropertyTagCache || (FindObject<UClass>(InAssetData.AssetClassPath) == nullptr), TEXT("FAssetPropertyTagCache not populated for type %s when fetching all attributes"), 
			*WriteToString<256>(InAssetData.AssetClassPath));*/

		OutAttributeValues.Reserve(OutAttributeValues.Num() + InAssetData.TagsAndValues.Num());
		for (const auto& TagAndValue : InAssetData.TagsAndValues)
		{
			FContentBrowserItemDataAttributeValue& GenericAttributeValue = OutAttributeValues.Add(TagAndValue.Key);
			GetGenericItemAttribute(TagAndValue.Key, TagAndValue.Value.AsString(), ClassPropertyTagCache, InIncludeMetaData, GenericAttributeValue);
			if (ParentClassPropertyTagCache && ParentClassPropertyTagCache->GetCacheForTag(TagAndValue.Key))
			{
				GetGenericItemAttribute(TagAndValue.Key, TagAndValue.Value.AsString(), ParentClassPropertyTagCache, InIncludeMetaData, GenericAttributeValue);
			}
		}
	}

	return true;
}

bool GetAssetAccessSpecifier(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		OutAssetAccessSpecifier = AssetPayload->GetAssetData().GetAssetAccessSpecifier();
		return true;
	}

	return false;
}

bool CanModifyAssetAccessSpecifier(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	check(InAssetTools);

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InOwnerDataSource, InItem))
	{
		FNameBuilder AssetPath;
		FTopLevelAssetPath(AssetPayload->GetAssetData().PackageName, AssetPayload->GetAssetData().AssetName).AppendString(AssetPath);

		return InAssetTools->CanAssetBePublic(AssetPath, nullptr);
	}

	return false;
}

void PopulateAssetFolderContextMenu(UContentBrowserDataSource* InOwnerDataSource, UToolMenu* InMenu, FAssetFolderContextMenu& InAssetFolderContextMenu)
{
	const UContentBrowserDataMenuContext_FolderMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FolderMenu was missing!"));

	// Extract the internal package paths that belong to this data source from the full list of selected items given in the context
	TArray<FString> SelectedPackagePaths;
	TArray<FString> SelectedAssetPackages;
	for (const FContentBrowserItem& SelectedItem : ContextObject->SelectedItems)
	{
		for (const FContentBrowserItemData& SelectedItemData : SelectedItem.GetInternalItems())
		{
			if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InOwnerDataSource, SelectedItemData))
			{
				SelectedPackagePaths.Add(FolderPayload->GetInternalPath().ToString());
			}
			else if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> ItemPayload = GetAssetFileItemPayload(InOwnerDataSource, SelectedItemData))
			{
				SelectedAssetPackages.Add(ItemPayload->GetAssetData().PackageName.ToString());
			}
		}
	}

	InAssetFolderContextMenu.MakeContextMenu(
		InMenu,
		SelectedPackagePaths,
		SelectedAssetPackages
		);
}

void PopulateAssetFileContextMenu(UContentBrowserDataSource* InOwnerDataSource, UToolMenu* InMenu, FAssetFileContextMenu& InAssetFileContextMenu)
{
	const UContentBrowserDataMenuContext_FileMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FileMenu was missing!"));

	const UContentBrowserAssetContextMenuContext* AssetContextMenu = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
	checkf(AssetContextMenu, TEXT("Required context UContentBrowserAssetContextMenuContext was missing!"));

	FAssetFileContextMenu::FOnShowAssetsInPathsView OnShowAssetsInPathsView = FAssetFileContextMenu::FOnShowAssetsInPathsView::CreateLambda([OwnerDataSource = TWeakObjectPtr<UContentBrowserDataSource>(InOwnerDataSource), OnShowInPathsView = ContextObject->OnShowInPathsView](const TArray<FAssetData>& InAssetsToShow)
	{
		UContentBrowserDataSource* OwnerDataSourcePtr = OwnerDataSource.Get();
		if (OwnerDataSourcePtr && OnShowInPathsView.IsBound())
		{
			TArray<FContentBrowserItem> ItemsToShow;
			for (const FAssetData& AssetToShow : InAssetsToShow)
			{
				FName VirtualPathToShow;
				if (OwnerDataSourcePtr->Legacy_TryConvertAssetDataToVirtualPath(AssetToShow, /*bUseFolderPaths*/false, VirtualPathToShow))
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					ItemsToShow.Emplace(
						CreateAssetFileItem(OwnerDataSourcePtr, VirtualPathToShow, AssetToShow.ObjectPath, AssetToShow));
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
			OnShowInPathsView.Execute(ItemsToShow);
		}
	});

	InAssetFileContextMenu.MakeContextMenu(
		InMenu,
		AssetContextMenu->SelectedAssets,
		OnShowAssetsInPathsView
	);
}

} // namespace ContentBrowserAssetData

#undef LOCTEXT_NAMESPACE
