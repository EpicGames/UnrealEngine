// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAssetDataPayload.h"
#include "AssetToolsModule.h"
#include "AssetThumbnail.h"
#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "Engine/Texture2D.h"
#include "IAssetTools.h"
#include "Materials/Material.h"
#include "IAssetTypeActions.h"
#include "Misc/PackageName.h"

namespace UE::ContentBrowserAssetDataSource::Private
{
	void GetFilenameFromPackageName(const FString& PackageName, FString& OutFileName, TFunctionRef<UPackage*()> GetPackage)
	{
		// Get the filename by finding it on disk first
		if (!FPackageName::DoesPackageExist(PackageName, &OutFileName))
		{
			if (const UPackage* Package = GetPackage())
			{
				// This is a package in memory that has not yet been saved. Determine the extension and convert to a filename
				const FString* PackageExtension = Package->ContainsMap() ? &FPackageName::GetMapPackageExtension() : &FPackageName::GetAssetPackageExtension();
				FPackageName::TryConvertLongPackageNameToFilename(PackageName, OutFileName, *PackageExtension);
			}
		}
	}
}

const FString& FContentBrowserAssetFolderItemDataPayload::GetFilename() const
{
	if (!bHasCachedFilename)
	{
		FPackageName::TryConvertLongPackageNameToFilename(InternalPath.ToString() / FString(), CachedFilename);
		bHasCachedFilename = true;
	}
	return CachedFilename;
}


FContentBrowserAssetFileItemDataPayload::FContentBrowserAssetFileItemDataPayload(FAssetData&& InAssetData)
	: AssetData(MoveTemp(InAssetData))
{	
}

FContentBrowserAssetFileItemDataPayload::FContentBrowserAssetFileItemDataPayload(const FAssetData& InAssetData)
	: AssetData(InAssetData)
{
}

UPackage* FContentBrowserAssetFileItemDataPayload::GetPackage(const bool bTryRecacheIfNull) const
{
	if (bHasCachedPackagePtr && CachedPackagePtr.IsStale())
	{
		FlushCaches();
	}
	
	if (!bHasCachedPackagePtr || (bTryRecacheIfNull && !CachedPackagePtr.IsValid()))
	{
		if (!AssetData.PackageName.IsNone())
		{
			CachedPackagePtr = FindObjectSafe<UPackage>(nullptr, *FNameBuilder(AssetData.PackageName), EFindObjectFlags::ExactClass);
		}
		bHasCachedPackagePtr = true;
	}
	return CachedPackagePtr.Get();
}

UPackage* FContentBrowserAssetFileItemDataPayload::LoadPackage(TSet<FName> LoadTags) const
{
	if (!bHasCachedPackagePtr || !CachedPackagePtr.IsValid())
	{
		if (!AssetData.PackageName.IsNone())
		{
			FLinkerInstancingContext InstancingContext(MoveTemp(LoadTags));
			CachedPackagePtr = ::LoadPackage(nullptr, *FNameBuilder(AssetData.PackageName), LOAD_None, nullptr, &InstancingContext);
			(void)GetAsset(/*bTryRecacheIfNull*/true); // Also re-cache the asset pointer
		}
		bHasCachedPackagePtr = true;
	}
	return CachedPackagePtr.Get();
}

UObject* FContentBrowserAssetFileItemDataPayload::GetAsset(const bool bTryRecacheIfNull) const
{
	if (bHasCachedAssetPtr && CachedAssetPtr.IsStale())
	{
		FlushCaches();
	}
	
	if (!bHasCachedAssetPtr || (bTryRecacheIfNull && !CachedAssetPtr.IsValid()))
	{
		if (AssetData.IsValid())
		{
			CachedAssetPtr = FindObjectSafe<UObject>(nullptr, *AssetData.GetObjectPathString());
		}
		bHasCachedAssetPtr = true;
	}
	return CachedAssetPtr.Get();
}

UObject* FContentBrowserAssetFileItemDataPayload::LoadAsset(TSet<FName> LoadTags) const
{
	if (!bHasCachedAssetPtr || !CachedAssetPtr.IsValid())
	{
		if (AssetData.IsValid())
		{
			FLinkerInstancingContext InstancingContext(MoveTemp(LoadTags));
			CachedAssetPtr = LoadObject<UObject>(nullptr, *AssetData.GetObjectPathString(), nullptr, LOAD_None, nullptr, &InstancingContext);
			(void)GetPackage(/*bTryRecacheIfNull*/true); // Also re-cache the package pointer
		}
		bHasCachedAssetPtr = true;
	}
	return CachedAssetPtr.Get();
}

TSharedPtr<IAssetTypeActions> FContentBrowserAssetFileItemDataPayload::GetAssetTypeActions() const
{
	if (!bHasCachedAssetTypeActionsPtr)
	{
		if (UClass* AssetClass = AssetData.GetClass())
		{
			static const FName NAME_AssetTools = "AssetTools";
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(NAME_AssetTools);
			CachedAssetTypeActionsPtr = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetClass);
		}
		bHasCachedAssetTypeActionsPtr = true;
	}
	return CachedAssetTypeActionsPtr.Pin();
}

const UAssetDefinition* FContentBrowserAssetFileItemDataPayload::GetAssetDefinition() const
{
	if (!bHasCachedAssetDefinitionPtr)
	{
		CachedAssetDefinitionPtr = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData);
		bHasCachedAssetDefinitionPtr = true;
	}
	return CachedAssetDefinitionPtr.Get();
}

const FString& FContentBrowserAssetFileItemDataPayload::GetFilename() const
{
	if (!bHasCachedFilename)
	{
		auto GetPackageLambda = [this]()
			{ 
				return GetPackage(); 
			};

		UE::ContentBrowserAssetDataSource::Private::GetFilenameFromPackageName(AssetData.PackageName.ToString(), CachedFilename, GetPackageLambda);

		bHasCachedFilename = true;
	}
	return CachedFilename;
}

void FContentBrowserAssetFileItemDataPayload::UpdateThumbnail(FAssetThumbnail& InThumbnail) const
{
	if (UObject* AssetPtr = GetAsset())
	{
		constexpr float TimeToBoostTextureResidency = 10.0f;
		if (UTexture2D* TextureAssetPtr = Cast<UTexture2D>(AssetPtr))
		{
			TextureAssetPtr->SetForceMipLevelsToBeResident(TimeToBoostTextureResidency);
		}
		else if (UMaterial* MaterialAssetPtr = Cast<UMaterial>(AssetPtr))
		{
			MaterialAssetPtr->SetForceMipLevelsToBeResident(false, false, TimeToBoostTextureResidency);
		}
	}

	InThumbnail.SetAsset(AssetData);
}

void FContentBrowserAssetFileItemDataPayload::FlushCaches() const
{
	bHasCachedPackagePtr = false;
	CachedPackagePtr.Reset();

	bHasCachedAssetPtr = false;
	CachedAssetPtr.Reset();

	bHasCachedAssetTypeActionsPtr = false;
	CachedAssetTypeActionsPtr.Reset();

	bHasCachedFilename = false;
	CachedFilename.Reset();
}

FContentBrowserAssetFileItemDataPayload_Creation::FContentBrowserAssetFileItemDataPayload_Creation(FAssetData&& InAssetData, UClass* InAssetClass, UFactory* InFactory)
	: FContentBrowserAssetFileItemDataPayload(MoveTemp(InAssetData))
	, AssetClass(InAssetClass)
	, Factory(InFactory)
{
}


FContentBrowserAssetFileItemDataPayload_Duplication::FContentBrowserAssetFileItemDataPayload_Duplication(FAssetData&& InAssetData, TWeakObjectPtr<UObject> InSourceObject)
	: FContentBrowserAssetFileItemDataPayload(MoveTemp(InAssetData))
	, SourceObject(InSourceObject)
{
}

FContentBrowserUnsupportedAssetFileItemDataPayload::FContentBrowserUnsupportedAssetFileItemDataPayload(FAssetData&& InAssetData)
	: OptionalAssetData(MakeUnique<FAssetData>(MoveTemp(InAssetData)))
{
}

FContentBrowserUnsupportedAssetFileItemDataPayload::FContentBrowserUnsupportedAssetFileItemDataPayload(const FAssetData& InAssetData)
	: OptionalAssetData(MakeUnique<FAssetData>(InAssetData))
{
}

const FAssetData* FContentBrowserUnsupportedAssetFileItemDataPayload::GetAssetDataIfAvailable() const
{
	return OptionalAssetData.Get();
}

const FString& FContentBrowserUnsupportedAssetFileItemDataPayload::GetFilename() const
{
	if (!bHasCachedFilename)
	{
		auto GetPackageLambda = [this]()
		{
			return GetPackage();
		};

		// Update this when we will show the asset that are to recent.
		const FString PackageName = OptionalAssetData ? OptionalAssetData->PackageName.ToString() : FString();

		UE::ContentBrowserAssetDataSource::Private::GetFilenameFromPackageName(PackageName, CachedFilename, GetPackageLambda);

		bHasCachedFilename = true;
	}
	return CachedFilename;
}

UPackage* FContentBrowserUnsupportedAssetFileItemDataPayload::GetPackage() const
{
	if (bHasCachedPackagePtr && CachedPackagePtr.IsStale())
	{
		FlushCaches();
	}

	if (!bHasCachedPackagePtr || !CachedPackagePtr.IsValid())
	{
		if (const FAssetData* AssetData = OptionalAssetData.Get())
		{ 
			if (!AssetData->PackageName.IsNone())
			{
				CachedPackagePtr = FindObjectSafe<UPackage>(nullptr, *FNameBuilder(AssetData->PackageName), EFindObjectFlags::ExactClass);
			}
		}
		bHasCachedPackagePtr = true;
	}
	return CachedPackagePtr.Get();
}

void FContentBrowserUnsupportedAssetFileItemDataPayload::FlushCaches() const
{
	bHasCachedPackagePtr = false;
	CachedPackagePtr.Reset();

	bHasCachedFilename = false;
	CachedFilename.Reset();
}
