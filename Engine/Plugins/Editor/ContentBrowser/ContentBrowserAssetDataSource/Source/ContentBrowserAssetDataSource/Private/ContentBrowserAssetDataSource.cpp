// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAssetDataSource.h"

#include "Algo/Transform.h"
#include "AssetPropertyTagCache.h"
#include "Async/ParallelFor.h"
#include "ContentBrowserAssetDataCore.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "CollectionManagerModule.h"
#include "ContentBrowserAssetDataPayload.h"
#include "ContentBrowserUtils.h"
#include "ICollectionContainer.h"
#include "AssetViewUtils.h"
#include "ContentBrowserItemPath.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/ObjectSaveContext.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Factories/Factory.h"
#include "HAL/FileManager.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "ToolMenus.h"
#include "Misc/Char.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "NewAssetContextMenu.h"
#include "AssetFolderContextMenu.h"
#include "AssetFileContextMenu.h"
#include "ContentBrowserAssetDataSourceCommands.h"
#include "ContentBrowserAssetDataSourceUtils.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDataUtils.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorDirectories.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/Paths.h"
#include "Widgets/Input/SButton.h"
#include "SActionButton.h"
#include "Subsystems/ImportSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Tasks/Task.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserAssetDataSource)

#define LOCTEXT_NAMESPACE "ContentBrowserAssetDataSource"

namespace AssetDataSource
{
	bool bAllowInternalParallelism = true;
	FAutoConsoleVariableRef CVarAllowInternalParallelism(
		TEXT("AssetDataSource.AllowInternalParallelism"),
		bAllowInternalParallelism,
		TEXT("Set to 0 to disable internal parallelism inside data source in case of threading issues."),
		ECVF_Default
	);

	bool bOptimizeEnumerateInMemoryAssets = true;
	FAutoConsoleVariableRef CVarOptimizeEnumerateInMemoryAssets(
		TEXT("AssetDataSource.OptimizeEnumerateInMemoryAssets"),
		bOptimizeEnumerateInMemoryAssets ,
		TEXT("1: Explicitly fetch fresh asset data for only new/dirty assets. 0: Fetch fresh asset data for all loaded assets."),
		ECVF_Default
	);
}

enum class EContentBrowserFolderAttributes : uint8
{
	/**
	 * No special attributes.
	 */
	None = 0,

	/**
	 * This folder should always be visible, even if it contains no content in the Content Browser view.
	 * This will include root content folders, and any folders that have been created directly (or indirectly) by a user action.
	 */
	AlwaysVisible = 1 << 0,

	/**
	 * This folder has non-redirector assets that will appear in the Content Browser view.
	 */
	HasAssets = 1 << 1,

	/**
	 * This folder has visible public content that will appear in the Content Browser view.
	 */
	HasVisiblePublicContent = 1 << 2,

	/**
	 * This folder has source (uncooked) content that will appear in the Content Browser view.
	 */
	HasSourceContent = 1 << 3,

	/**
	 * This folder is inside a plugin.
	 */
	IsInPlugin = 1 << 4,

	/**
	 * This folder has redirector assets that will appear in the Content Browser view if the UI wishes to display them
	 */
	HasRedirectors = 1 << 5,
};
ENUM_CLASS_FLAGS(EContentBrowserFolderAttributes);

// Produce a string of flags |'d together for logging
FStringBuilderBase& operator<<(FStringBuilderBase& Builder, EContentBrowserFolderAttributes Attribs)
{
	bool bFirst = true;
	for (EContentBrowserFolderAttributes Flag : MakeFlagsRange(Attribs))
	{
		if (!bFirst)
		{
			Builder << TEXTVIEW("|");
		}
		switch (Flag)
		{
			case EContentBrowserFolderAttributes::AlwaysVisible:
				Builder << TEXTVIEW("AlwaysVisible");
				break;
			case EContentBrowserFolderAttributes::HasAssets:
				Builder << TEXTVIEW("HasAssets");
				break;
			case EContentBrowserFolderAttributes::HasVisiblePublicContent:
				Builder << TEXTVIEW("HasVisiblePublicContent");
				break;
			case EContentBrowserFolderAttributes::HasSourceContent:
				Builder << TEXTVIEW("HasSourceContent");
				break;
			case EContentBrowserFolderAttributes::IsInPlugin:
				Builder << TEXTVIEW("IsInPlugin");
				break;
			case EContentBrowserFolderAttributes::HasRedirectors:
				Builder << TEXTVIEW("HasRedirectors");
				break;
			default:
				Builder << TEXTVIEW("Unknown");
				break;
		}
	}
	return Builder;
}

UContentBrowserAssetDataSource::FOnAssetDataSourcePathAdded UContentBrowserAssetDataSource::OnAssetPathAddedDelegate;
UContentBrowserAssetDataSource::FOnAssetDataSourcePathRemoved UContentBrowserAssetDataSource::OnAssetPathRemovedDelegate;

void UContentBrowserAssetDataSource::Initialize(const bool InAutoRegister)
{
	check(GIsEditor && !IsRunningCommandlet());

	Super::Initialize(InAutoRegister);

	AssetRegistry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	AssetRegistry->OnFileLoadProgressUpdated().AddUObject(this, &UContentBrowserAssetDataSource::OnAssetRegistryFileLoadProgress);
	AssetTools = &FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	ContentBrowserModule = &FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// Extend content browser commands
	TArray<FContentBrowserCommandExtender>& CommandExtenders = ContentBrowserModule->GetAllContentBrowserCommandExtenders();
	CommandExtenders.Add(FContentBrowserCommandExtender::CreateUObject(this, &UContentBrowserAssetDataSource::OnExtendContentBrowserCommands));

	CollectionManager = &FCollectionManagerModule::GetModule().Get();

	// Listen for asset registry updates
	AssetRegistry->OnAssetsAdded().AddUObject(this, &UContentBrowserAssetDataSource::OnAssetsAdded);
	AssetRegistry->OnAssetRemoved().AddUObject(this, &UContentBrowserAssetDataSource::OnAssetRemoved);
	AssetRegistry->OnAssetRenamed().AddUObject(this, &UContentBrowserAssetDataSource::OnAssetRenamed);
	AssetRegistry->OnAssetUpdated().AddUObject(this, &UContentBrowserAssetDataSource::OnAssetUpdated);
	AssetRegistry->OnAssetUpdatedOnDisk().AddUObject(this, &UContentBrowserAssetDataSource::OnAssetUpdatedOnDisk);
	AssetRegistry->OnPathsAdded().AddUObject(this, &UContentBrowserAssetDataSource::OnPathsAdded);
	AssetRegistry->OnPathsRemoved().AddUObject(this, &UContentBrowserAssetDataSource::OnPathsRemoved);

	// Listen for when assets are loaded or changed
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UContentBrowserAssetDataSource::OnObjectPropertyChanged);
	
	// Listen for when assets are saved, listerns are notified in time despite presave because we queue updates for later processing 
	FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UContentBrowserAssetDataSource::OnObjectPreSave);

	// Listen for module initialization to update FAssetPropertyTagCache
	FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.AddWeakLambda(this, [](FName, ECompiledInUObjectsRegisteredStatus){
		FAssetPropertyTagCache::Get().CachePendingClasses();
	});

	// Listen for classes being loaded 
	FCoreUObjectDelegates::OnAssetLoaded.AddWeakLambda(this, [](UObject* Object){
		if (UClass* Class = Cast<UClass>(Object))
		{
			FAssetPropertyTagCache::Get().TryCacheClass(FTopLevelAssetPath(Class));
		}
	});

	// Listen for new mount roots
	FPackageName::OnContentPathMounted().AddUObject(this, &UContentBrowserAssetDataSource::OnContentPathMounted);
	FPackageName::OnContentPathDismounted().AddUObject(this, &UContentBrowserAssetDataSource::OnContentPathDismounted);

	// Listen for paths being forced visible
	AssetViewUtils::OnAlwaysShowPath().AddUObject(this, &UContentBrowserAssetDataSource::OnAlwaysShowPath);

	// Register our ability to create assets via the legacy Content Browser API
	ContentBrowserDataLegacyBridge::OnCreateNewAsset().BindUObject(this, &UContentBrowserAssetDataSource::OnBeginCreateAsset);

	// Create the asset menu instances
	AssetFolderContextMenu = MakeShared<FAssetFolderContextMenu>();
	AssetFileContextMenu = MakeShared<FAssetFileContextMenu>();

	// Bind the asset specific menu extensions
	{
		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AddNewContextMenu"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserAssetDataSource>(this)](UToolMenu* InMenu)
				{
					if (UContentBrowserAssetDataSource* This = WeakThis.Get())
					{
						This->PopulateAddNewContextMenu(InMenu);
					}
				}));
		}


		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.ToolBar"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserAssetDataSource>(this)](UToolMenu* InMenu)
				{
					if (UContentBrowserAssetDataSource* This = WeakThis.Get())
					{
						This->PopulateContentBrowserToolBar(InMenu);
					}
				}));
		}


		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserAssetDataSource>(this)](UToolMenu* InMenu)
				{
					if (UContentBrowserAssetDataSource* This = WeakThis.Get())
					{
						This->PopulateAssetFolderContextMenu(InMenu);
					}
				}));
		}

		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserAssetDataSource>(this)](UToolMenu* InMenu)
				{
					if (UContentBrowserAssetDataSource* This = WeakThis.Get())
					{
						This->PopulateAssetFileContextMenu(InMenu);
					}
				}));
		}

		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.DragDropContextMenu"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserAssetDataSource>(this)](UToolMenu* InMenu)
				{
					if (UContentBrowserAssetDataSource* This = WeakThis.Get())
					{
						This->PopulateDragDropContextMenu(InMenu);
					}
				}));
		}
	}

	DiscoveryStatusText = LOCTEXT("InitializingAssetDiscovery", "Initializing Asset Discovery...");

	FAssetPropertyTagCache& PropertyTagCache = FAssetPropertyTagCache::Get();

	// Populate the initial set of folder attributes
	// This will be updated as the scan finds more content
	AssetRegistry->EnumerateAllCachedPaths([this](FName PathName) { 
		FNameBuilder NameBuilder{PathName};
		OnPathsAdded({NameBuilder.ToView()});
		return true; 
	});
	AssetRegistry->EnumerateAllAssets([this, &PropertyTagCache ](const FAssetData& InAssetData)
		{
			if (InAssetData.GetOptionalOuterPathName().IsNone())
			{
				PropertyTagCache.TryCacheClass(InAssetData.AssetClassPath);
			}
			OnPathPopulated(InAssetData);
			return true;
		}, UE::AssetRegistry::EEnumerateAssetsFlags::OnlyOnDiskAssets);
	RecentlyPopulatedAssetFolders.Empty();

	FPackageName::QueryRootContentPaths(RootContentPaths);

	BuildRootPathVirtualTree();

	for (const FString& RootContentPath : RootContentPaths)
	{
		// Mount roots are always visible
		OnAlwaysShowPath(RootContentPath);
		
		// Populate the acceleration structure
		AddRootContentPathToStateMachine(RootContentPath);
	}
}

void UContentBrowserAssetDataSource::Shutdown()
{
	CollectionManager = nullptr;

	AssetTools = nullptr;
	AssetRegistry = nullptr;

	RootContentPaths.Empty();
	RootContentPathsTrie.NextNodes.Empty();

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(AssetRegistryConstants::ModuleName))
	{
		IAssetRegistry* AssetRegistryMaybe = AssetRegistryModule->TryGet();
		if (AssetRegistryMaybe)
		{
			AssetRegistryMaybe->OnFileLoadProgressUpdated().RemoveAll(this);

			AssetRegistryMaybe->OnAssetsAdded().RemoveAll(this);
			AssetRegistryMaybe->OnAssetRemoved().RemoveAll(this);
			AssetRegistryMaybe->OnAssetRenamed().RemoveAll(this);
			AssetRegistryMaybe->OnAssetUpdated().RemoveAll(this);
			AssetRegistryMaybe->OnAssetUpdatedOnDisk().RemoveAll(this);	
			AssetRegistryMaybe->OnPathsAdded().RemoveAll(this);
			AssetRegistryMaybe->OnPathsRemoved().RemoveAll(this);
			AssetRegistryMaybe->OnFilesLoaded().RemoveAll(this);
		}
	}
	
	if (FContentBrowserModule* ContentBrowserModulePtr = FModuleManager::GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser")))
	{
		TArray<FContentBrowserCommandExtender>& CommandExtenders = ContentBrowserModulePtr->GetAllContentBrowserCommandExtenders();
		CommandExtenders.RemoveAll([this](const FContentBrowserCommandExtender& Delegate) { return Delegate.GetUObject() == this; });
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);

	AssetViewUtils::OnAlwaysShowPath().RemoveAll(this);

	ContentBrowserDataLegacyBridge::OnCreateNewAsset().Unbind();

	Super::Shutdown();
}

bool UContentBrowserAssetDataSource::PopulateAssetFilterInputParams(FAssetFilterInputParams& Params, UContentBrowserDataSource* DataSource, IAssetRegistry* InAssetRegistry, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, ICollectionManager* CollectionManager, FAssetDataSourceFilterCache* InFilterCache)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Params.CollectionManager = CollectionManager;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Params.AssetFilterCache = InFilterCache;

	Params.bIncludeFolders = EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders);
	Params.bIncludeFiles = EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles);
	Params.bIncludeAssets = EnumHasAnyFlags(InFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeAssets);
	Params.bIncludeRedirectors = EnumHasAnyFlags(InFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeRedirectors);

	// Everything this data source tracks is either an asset or a redirector
	if (!Params.bIncludeAssets && !Params.bIncludeRedirectors)
	{
		return false;
	}

	// Everything this data source tracks is either a file or a folder
	if (!Params.bIncludeFolders && !Params.bIncludeFiles)
	{
		return false;
	}

	Params.CollectionFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataCollectionFilter>();

	Params.ObjectFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataObjectFilter>();
	Params.PackageFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataPackageFilter>();
	Params.ClassFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataClassFilter>();

	Params.PathPermissionList = Params.PackageFilter && Params.PackageFilter->PathPermissionList && Params.PackageFilter->PathPermissionList->HasFiltering() ? Params.PackageFilter->PathPermissionList.Get() : nullptr;
	Params.ClassPermissionList = Params.ClassFilter && Params.ClassFilter->ClassPermissionList && Params.ClassFilter->ClassPermissionList->HasFiltering() ? Params.ClassFilter->ClassPermissionList.Get() : nullptr;

	// If we are filtering all paths, then we can bail now as we won't return any content
	if (Params.PathPermissionList && Params.PathPermissionList->IsDenyListAll())
	{
		return false;
	}

	Params.DataSource = DataSource;
	Params.AssetRegistry = InAssetRegistry;
	Params.FilterList = &OutCompiledFilter.CompiledFilters.FindOrAdd(DataSource);
	Params.AssetDataFilter = &Params.FilterList->FindOrAddFilter<FContentBrowserCompiledAssetDataFilter>();
	Params.AssetDataFilter->bFilterExcludesAllAssets = true;
	Params.AssetDataFilter->ItemAttributeFilter = InFilter.ItemAttributeFilter;
	Params.AssetDataFilter->ItemCategoryFilter = InFilter.ItemCategoryFilter;
	Params.InternalPaths.Reset();

	Params.UnsupportedClassFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataUnsupportedClassFilter>();
	if (Params.UnsupportedClassFilter && Params.UnsupportedClassFilter->ClassPermissionList && Params.UnsupportedClassFilter->ClassPermissionList->HasFiltering())
	{
		Params.ConvertToUnsupportedAssetDataFilter = &Params.FilterList->FindOrAddFilter<FContentBrowserCompiledUnsupportedAssetDataFilter>();
	}

	return true;
}

bool UContentBrowserAssetDataSource::CreatePathFilter(FAssetFilterInputParams& Params, const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, FSubPathEnumerationFunc SubPathEnumeration)
{
	Params.AssetDataFilter->bFilterExcludesAllAssets = true;
	Params.AssetDataFilter->ItemAttributeFilter = InFilter.ItemAttributeFilter;

	FName ConvertedPath;
	const EContentBrowserPathType ConvertedPathType = Params.DataSource->TryConvertVirtualPath(InPath, ConvertedPath);

	if (ConvertedPathType == EContentBrowserPathType::Internal)
	{
		Params.InternalPaths.Add(ConvertedPath);
	}
	else if (ConvertedPathType != EContentBrowserPathType::Virtual)
	{
		return false;
	}

	if (Params.bIncludeFolders)
	{
		// If we're including folders, but not doing a recursive search then we need to handle that here as the asset code below can't deal with that correctly
		// We also go through this path if we're not including files, as then we don't run the asset code below
		if (!InFilter.bRecursivePaths || !Params.bIncludeFiles)
		{
			// Build the basic paths permissions from the given data
			if (Params.PackageFilter)
			{
				Params.AssetDataFilter->bRecursivePackagePathsToInclude = Params.PackageFilter->bRecursivePackagePathsToInclude;
				for (const FName PackagePathToInclude : Params.PackageFilter->PackagePathsToInclude)
				{
					Params.AssetDataFilter->PackagePathsToInclude.AddAllowListItem(NAME_None, PackagePathToInclude);
				}

				Params.AssetDataFilter->bRecursivePackagePathsToExclude = Params.PackageFilter->bRecursivePackagePathsToExclude;
				for (const FName PackagePathToExclude : Params.PackageFilter->PackagePathsToExclude)
				{
					Params.AssetDataFilter->PackagePathsToExclude.AddDenyListItem(NAME_None, PackagePathToExclude);
				}
			}
			if (Params.PathPermissionList)
			{
				Params.AssetDataFilter->PathPermissionList = *Params.PathPermissionList;
			}
		}

		// Recursive caching of folders is at least as slow as running the query on-demand
		// and significantly slower when only querying the status of a few updated items
		// To this end, we only attempt to pre-cache non-recursive queries
		if (InFilter.bRecursivePaths)
		{
			Params.AssetDataFilter->bRunFolderQueryOnDemand = true;
			Params.AssetDataFilter->VirtualPathToScanOnDemand = InPath.ToString();
		}
		else
		{
			if (ConvertedPathType == EContentBrowserPathType::Internal)
			{
				SubPathEnumeration(ConvertedPath, [&Params](FName SubPath)
				{
					if (UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(*Params.AssetDataFilter, SubPath))
					{
						Params.AssetDataFilter->CachedSubPaths.Add(SubPath);
					}

					return true;
				}, false);
			}
			else if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				FContentBrowserCompiledVirtualFolderFilter* VirtualFolderFilter = nullptr;
				Params.DataSource->GetRootPathVirtualTree().EnumerateSubPaths(InPath, [&Params, &VirtualFolderFilter](FName VirtualSubPath, FName InternalSubPath)
				{
					if (!InternalSubPath.IsNone())
					{
						if (UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(*Params.AssetDataFilter, InternalSubPath))
						{
							Params.AssetDataFilter->CachedSubPaths.Add(InternalSubPath);
						}
					}
					else
					{
						// Determine if any internal path under VirtualSubPath passes
						bool bPassesFilter = false;
						Params.DataSource->GetRootPathVirtualTree().EnumerateSubPaths(VirtualSubPath, [&Params, &VirtualFolderFilter, &bPassesFilter](FName RecursiveVirtualSubPath, FName RecursiveInternalSubPath)
						{
							bPassesFilter = bPassesFilter || (!RecursiveInternalSubPath.IsNone() && UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(*Params.AssetDataFilter, RecursiveInternalSubPath));
							return bPassesFilter == false;
						}, true);

						if (bPassesFilter)
						{
							if (!VirtualFolderFilter)
							{
								VirtualFolderFilter = &Params.FilterList->FindOrAddFilter<FContentBrowserCompiledVirtualFolderFilter>();
							}

							if (!VirtualFolderFilter->CachedSubPaths.Contains(VirtualSubPath))
							{
								VirtualFolderFilter->CachedSubPaths.Add(VirtualSubPath, Params.DataSource->CreateVirtualFolderItem(VirtualSubPath));
							}
						}
					}

					return true;
				}, false);
			}
		}
	}
	else if (Params.bIncludeFiles)
	{
		if (InFilter.bRecursivePaths)
		{
			if (ConvertedPathType == EContentBrowserPathType::Internal)
			{
				// Nothing more to do, Params.InternalPaths already contains ConvertedPath
			}
			else if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				// Include all internal mounts that pass recursively
				Params.DataSource->GetRootPathVirtualTree().EnumerateSubPaths(InPath, [&Params](FName VirtualSubPath, FName InternalSubPath)
				{
					if (!InternalSubPath.IsNone() && UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(*Params.AssetDataFilter, InternalSubPath))
					{
						Params.InternalPaths.Add(InternalSubPath);
					}
					return true;
				}, true);

				if (Params.InternalPaths.Num() == 0)
				{
					// No internal folders found in the hierarchy of virtual path that passed, there will be no files
					return false;
				}
			}
		}
		else
		{
			if (ConvertedPathType == EContentBrowserPathType::Internal)
			{
				// Nothing more to do, Params.InternalPaths already contains ConvertedPath
			}
			else if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				// There are no files directly contained by a dynamically generated fully virtual folder
				return false;
			}
		}
	}

	return true;
}

bool UContentBrowserAssetDataSource::CreateAssetFilter(FAssetFilterInputParams& Params, FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, FSubPathEnumerationFunc* GetSubPackagePathsFunc, FCollectionEnumerationFunc* GetCollectionObjectPathsFunc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::CreateAssetFilter);

	// If we're not including files, then we can bail now as the rest of this function deals with assets
	if (!Params.bIncludeFiles)
	{
		return false;
	}

	// If we are filtering all classes, then we can bail now as we won't return any content
	if (Params.ClassPermissionList && Params.ClassPermissionList->IsDenyListAll() && !Params.UnsupportedClassFilter)
	{
		return false;
	}

	// If we are filtering out this path, then we can bail now as it won't return any content
	if (Params.PathPermissionList && !InFilter.bRecursivePaths)
	{
		for (auto It = Params.InternalPaths.CreateIterator(); It; ++It)
		{
			if (!Params.PathPermissionList->PassesStartsWithFilter(*It))
			{
				It.RemoveCurrent();
			}
		}

		if (Params.InternalPaths.Num() == 0)
		{
			return false;
		}
	}

	auto DefaultEnumarePackagePaths = [&Params](FName InPath, TFunctionRef<bool(FName)> Callback, bool bIsRecursive)
		{
			Params.AssetRegistry->EnumerateSubPaths(InPath
				, [&Callback](FName ChildPath)
					{
						return Callback(ChildPath);
					}
				, bIsRecursive);
		};

	FSubPathEnumerationFunc EnumeratePackagePaths = GetSubPackagePathsFunc ? *GetSubPackagePathsFunc : DefaultEnumarePackagePaths;

	// Build inclusive asset filter
	FARCompiledFilter CompiledInclusiveFilter;
	{
		// Build the basic inclusive filter from the given data
		{
			FARFilter InclusiveFilter;
			if (Params.ObjectFilter)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InclusiveFilter.ObjectPaths.Append(Params.ObjectFilter->ObjectNamesToInclude);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				InclusiveFilter.TagsAndValues.Append(Params.ObjectFilter->TagsAndValuesToInclude);
				InclusiveFilter.bIncludeOnlyOnDiskAssets |= Params.ObjectFilter->bOnDiskObjectsOnly;
			}
			if (Params.PackageFilter)
			{
				InclusiveFilter.PackageNames.Append(Params.PackageFilter->PackageNamesToInclude);
				InclusiveFilter.PackagePaths.Append(Params.PackageFilter->PackagePathsToInclude);
				if (Params.PackageFilter->bRecursivePackagePathsToInclude)
				{
					for (const FName Path : Params.PackageFilter->PackagePathsToInclude)
					{
						EnumeratePackagePaths(Path, [&InclusiveFilter](FName ChildPath)
							{
								InclusiveFilter.PackagePaths.Add(ChildPath);
								return true;
							}
							, Params.PackageFilter->bRecursivePackagePathsToInclude);
					}
				}
			}
			if (Params.ClassFilter)
			{
				InclusiveFilter.ClassPaths.Append(Params.ClassFilter->ClassNamesToInclude);
				InclusiveFilter.bRecursiveClasses |= Params.ClassFilter->bRecursiveClassNamesToInclude;
			}
			if (Params.CollectionFilter)
			{
				TArray<FSoftObjectPath> ObjectPathsForCollections;
				if (GetObjectPathsForCollections(Params.CollectionFilter->Collections, Params.CollectionFilter->bIncludeChildCollections, GetCollectionObjectPathsFunc, ObjectPathsForCollections) &&
					ObjectPathsForCollections.Num() == 0)
				{
					// If we had collections but they contained no objects then we can bail as nothing will pass the filter
					return false;
				}
				InclusiveFilter.SoftObjectPaths.Append(MoveTemp(ObjectPathsForCollections));
			}

#if DO_ENSURE
			// Ensure paths do not have trailing slash	
			static const FName RootPath = "/";

			for (const FName ItPath : Params.InternalPaths)
			{
				ensure(ItPath == RootPath || !FStringView(FNameBuilder(ItPath)).EndsWith(TEXT('/')));
			}

			for (const FName ItPath : InclusiveFilter.PackagePaths)
			{
				ensure(ItPath == RootPath || !FStringView(FNameBuilder(ItPath)).EndsWith(TEXT('/')));
			}
#endif // DO_ENSURE

			Params.AssetRegistry->CompileFilter(InclusiveFilter, CompiledInclusiveFilter);
		}


		// Add the backend class filtering to the unsupported asset filtering before the class permission are added
		if (Params.ConvertToUnsupportedAssetDataFilter)
		{
			if (Params.UnsupportedClassFilter)
			{
				if (const FPathPermissionList* ClassPermissionList = Params.UnsupportedClassFilter->ClassPermissionList.Get())
				{
					if (ClassPermissionList->HasFiltering())
					{
						if (Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.IsEmpty())
						{
							Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths = CompiledInclusiveFilter.ClassPaths;
						}
						else
						{
							Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths = Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths.Intersect(CompiledInclusiveFilter.ClassPaths);
						}
					}
				}
			}
		}


		// Remove any inclusive paths that aren't under the set of internal paths that we want to enumerate
		{
			FARCompiledFilter CompiledInternalPathFilter;

			if (!Params.AssetFilterCache || !Params.AssetFilterCache->GetCachedCompiledInternalPaths(InFilter, InPath, CompiledInternalPathFilter.PackagePaths))
			{
				/**
				 * This filter is created by testing the paths while we are recursively going down the path hierarchy
				 * This effective because it can stop exploring a sub part of the path three when the current path fail the attribute filter
				 * Also after a certain depth it stop testing the paths against the attribute filter since the result will the same as the parent path
				 */
				TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::CreateAssetFilter::CreateInternalPathFilter);
			
				CompiledInternalPathFilter.PackagePaths.Reserve(Params.InternalPaths.Num());

				const int32 MaxDepthPathTestNeeded = ContentBrowserDataUtils::GetMaxFolderDepthRequiredForAttributeFilter();

				// This builder is shared across calls because it is stack allocated and it could cause some issues in depth recursive calls
				FNameBuilder PathBufferStr;
				TFunction<void (FName, int32)> TestAndGatherChildPaths;
				TestAndGatherChildPaths = [&InFilter, &CompiledInternalPathFilter, &TestAndGatherChildPaths, &EnumeratePackagePaths, &PathBufferStr, MaxDepthPathTestNeeded](FName ChildPath, int32 CurrentDepth)
					{
						PathBufferStr.Reset();
						ChildPath.AppendString(PathBufferStr);
						if (ContentBrowserDataUtils::PathPassesAttributeFilter(PathBufferStr, CurrentDepth, InFilter.ItemAttributeFilter))
						{
							CompiledInternalPathFilter.PackagePaths.Add(ChildPath);
							++CurrentDepth;

							if (CurrentDepth < MaxDepthPathTestNeeded)
							{
								constexpr bool bIsRecursive = false;
								EnumeratePackagePaths(ChildPath
									, [&TestAndGatherChildPaths, CurrentDepth](FName ChildPath)
									{
										TestAndGatherChildPaths(ChildPath, CurrentDepth);
										return true;
									}
								, bIsRecursive);
							}
							else
							{
								constexpr bool bIsRecursive = true;
								EnumeratePackagePaths(ChildPath
									, [&CompiledInternalPathFilter](FName ChildPath)
									{
										CompiledInternalPathFilter.PackagePaths.Add(ChildPath);
										return true;
									}
								, bIsRecursive);
							}
						}
					};


				for (const FName InternalPath : Params.InternalPaths)
				{
					PathBufferStr.Reset();
					InternalPath.AppendString(PathBufferStr);
					FStringView Path(PathBufferStr);
					if (ContentBrowserDataUtils::PathPassesAttributeFilter(Path, 0, InFilter.ItemAttributeFilter))
					{
						CompiledInternalPathFilter.PackagePaths.Add(InternalPath);

						if (InFilter.bRecursivePaths)
						{
							// Minus one because the test depth start at zero
							const int32 CurrentDepth = ContentBrowserDataUtils::CalculateFolderDepthOfPath(Path) - 1;
							if (CurrentDepth < MaxDepthPathTestNeeded)
							{
								constexpr bool bIsRecursive = false;
								EnumeratePackagePaths(InternalPath
									, [&TestAndGatherChildPaths, CurrentDepth](FName ChildPath)
									{
										TestAndGatherChildPaths(ChildPath, CurrentDepth);
										return true;
									}
								, bIsRecursive);
							}
							else
							{
								constexpr bool bIsRecursive = true;
								EnumeratePackagePaths(InternalPath
									, [&CompiledInternalPathFilter](FName ChildPath)
									{
										CompiledInternalPathFilter.PackagePaths.Add(ChildPath);
										return true;
									}
								, bIsRecursive);
							}
						}
					}
				}
			
				if (Params.AssetFilterCache)
				{
					Params.AssetFilterCache->CacheCompiledInternalPaths(InFilter, InPath, CompiledInternalPathFilter.PackagePaths);
				}
			}

			if (CompiledInclusiveFilter.PackagePaths.Num() > 0)
			{
				// Explicit paths given - remove anything not in the internal paths set
				// If the paths resolve as empty then the combined filter will return nothing and can be skipped
				CompiledInclusiveFilter.PackagePaths = CompiledInclusiveFilter.PackagePaths.Intersect(CompiledInternalPathFilter.PackagePaths);
				if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
				{
					return false;
				}
			}
			else
			{
				// No explicit paths given - just use the internal paths set
				CompiledInclusiveFilter.PackagePaths = MoveTemp(CompiledInternalPathFilter.PackagePaths);
			}
		}

		// Add the backend class filtering to the unsupported asset filtering before the class permission are added
		if (Params.ConvertToUnsupportedAssetDataFilter)
		{
			if (Params.UnsupportedClassFilter)
			{
				if (const FPathPermissionList* ClassPermissionList = Params.UnsupportedClassFilter->ClassPermissionList.Get())
				{
					if (ClassPermissionList->HasFiltering())
					{
						Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths.Append(CompiledInclusiveFilter.ClassPaths);
					}
				}
			}
		}

		// Remove any inclusive paths that aren't in the explicit AllowList set
		if (Params.PathPermissionList && Params.PathPermissionList->HasAllowListEntries())
		{
			FARCompiledFilter CompiledPathFilterAllowList;
			{
				TArray<FString> AllowList = Params.PathPermissionList->GetAllowListEntries();
				CompiledPathFilterAllowList.PackagePaths.Reserve(AllowList.Num());
				for (const FString& AllowListEntry : AllowList)
				{
					FName PackageName{AllowListEntry};
					CompiledPathFilterAllowList.PackagePaths.Add(PackageName);

					constexpr bool bIsRecursive = true;
					EnumeratePackagePaths(PackageName, [&CompiledPathFilterAllowList](FName ChildPath)
							{
								CompiledPathFilterAllowList.PackagePaths.Add(ChildPath);
								return true;
							}
							, bIsRecursive);

				}
			}

			if (CompiledInclusiveFilter.PackagePaths.Num() > 0)
			{
				// Explicit paths given - remove anything not in the allow list paths set
				// If the paths resolve as empty then the combined filter will return nothing and can be skipped
				CompiledInclusiveFilter.PackagePaths = CompiledInclusiveFilter.PackagePaths.Intersect(CompiledPathFilterAllowList.PackagePaths);
				if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
				{
					return false;
				}
			}
			else
			{
				// No explicit paths given - just use the allow list paths set
				CompiledInclusiveFilter.PackagePaths = MoveTemp(CompiledPathFilterAllowList.PackagePaths);
			}
		}

		// Remove any inclusive classes that aren't in the explicit allow list set
		if (Params.ClassPermissionList && Params.ClassPermissionList->HasAllowListEntries())
		{
			FARCompiledFilter CompiledClassFilterAllowList;
			{
				FARFilter AllowListClassFilter;
				TArray<FString> AllowList = Params.ClassPermissionList->GetAllowListEntries();
				for (const FString& Path : AllowList)
				{
					AllowListClassFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
				}
				Params.AssetRegistry->CompileFilter(AllowListClassFilter, CompiledClassFilterAllowList);
			}

			if (CompiledInclusiveFilter.ClassPaths.Num() > 0)
			{
				// Explicit classes given - remove anything not in the allow list class set
				// If the classes resolve as empty then the combined filter will return nothing and can be skipped
				CompiledInclusiveFilter.ClassPaths = CompiledInclusiveFilter.ClassPaths.Intersect(CompiledClassFilterAllowList.ClassPaths);
				if (CompiledInclusiveFilter.ClassPaths.Num() == 0 && !Params.ConvertToUnsupportedAssetDataFilter)
				{
					return false;
				}
			}
			else
			{
				// No explicit classes given - just use the allow list class set
				CompiledInclusiveFilter.ClassPaths = MoveTemp(CompiledClassFilterAllowList.ClassPaths);
			}
		}
	}

	// Build exclusive asset filter
	FARCompiledFilter CompiledExclusiveFilter;
	{
		// Build the basic exclusive filter from the given data
		{
			FARFilter ExclusiveFilter;
			if (Params.ObjectFilter)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ExclusiveFilter.ObjectPaths.Append(Params.ObjectFilter->ObjectNamesToExclude);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				ExclusiveFilter.TagsAndValues.Append(Params.ObjectFilter->TagsAndValuesToExclude);
				ExclusiveFilter.bIncludeOnlyOnDiskAssets |= Params.ObjectFilter->bOnDiskObjectsOnly;
			}
			if (Params.PackageFilter)
			{
				ExclusiveFilter.PackageNames.Append(Params.PackageFilter->PackageNamesToExclude);
				ExclusiveFilter.PackagePaths.Append(Params.PackageFilter->PackagePathsToExclude);
				if (Params.PackageFilter->bRecursivePackagePathsToExclude)
				{
					for (const FName Path : Params.PackageFilter->PackagePathsToExclude)
					{
						EnumeratePackagePaths(Path, [&ExclusiveFilter](FName ChildPath)
							{
								ExclusiveFilter.PackagePaths.Add(ChildPath);
								return true;
							}
							, Params.PackageFilter->bRecursivePackagePathsToExclude);
					}
				}
			}
			if (Params.ClassFilter)
			{
				ExclusiveFilter.ClassPaths.Append(Params.ClassFilter->ClassNamesToExclude);
				ExclusiveFilter.bRecursiveClasses |= Params.ClassFilter->bRecursiveClassNamesToExclude;
			}

			if (!Params.bIncludeRedirectors)
			{
				ExclusiveFilter.ClassPaths.Add(FTopLevelAssetPath(UObjectRedirector::StaticClass()));
			}

			Params.AssetRegistry->CompileFilter(ExclusiveFilter, CompiledExclusiveFilter);
		}

		// Add any exclusive paths that are in the explicit DenyList set
		if (Params.PathPermissionList && Params.PathPermissionList->HasDenyListEntries())
		{
			TArray<FString> DenyListEntries = Params.PathPermissionList->GetDenyListEntries();
			CompiledExclusiveFilter.PackagePaths.Reserve(DenyListEntries.Num());
			for (const FString& PathString : DenyListEntries)
			{
				FName Path{PathString};
				CompiledExclusiveFilter.PackagePaths.Add(Path);

				constexpr bool bIsRecursive = true;
				EnumeratePackagePaths(Path, [&CompiledExclusiveFilter](FName ChildPath)
					{
						CompiledExclusiveFilter.PackagePaths.Add(ChildPath);
						return true;
					}
					, bIsRecursive);
			}
		}

		// Add any exclusive classes that are in the explicit DenyList set
		if (Params.ClassPermissionList && Params.ClassPermissionList->HasDenyListEntries())
		{
			FARCompiledFilter CompiledClassFilter;
			{
				FARFilter ClassFilter;
				for (const FString& Path : Params.ClassPermissionList->GetDenyListEntries())
				{
					ClassFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
				}
				Params.AssetRegistry->CompileFilter(ClassFilter, CompiledClassFilter);
			}

			CompiledExclusiveFilter.ClassPaths.Append(CompiledClassFilter.ClassPaths);
		}
	}

	// Apply our exclusive filter to the inclusive one to resolve cases where the exclusive filter cancels out the inclusive filter
	// If any filter components resolve as empty then the combined filter will return nothing and can be skipped
	{
		if (CompiledInclusiveFilter.PackageNames.Num() > 0 && CompiledExclusiveFilter.PackageNames.Num() > 0)
		{
			CompiledInclusiveFilter.PackageNames = CompiledInclusiveFilter.PackageNames.Difference(CompiledExclusiveFilter.PackageNames);
			if (CompiledInclusiveFilter.PackageNames.Num() == 0)
			{
				return false;
			}
			CompiledExclusiveFilter.PackageNames.Reset();
		}
		if (CompiledInclusiveFilter.PackagePaths.Num() > 0 && CompiledExclusiveFilter.PackagePaths.Num() > 0)
		{
			CompiledInclusiveFilter.PackagePaths = CompiledInclusiveFilter.PackagePaths.Difference(CompiledExclusiveFilter.PackagePaths);
			if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
			{
				return false;
			}
			CompiledExclusiveFilter.PackagePaths.Reset();
		}
		if (CompiledInclusiveFilter.SoftObjectPaths.Num() > 0 && CompiledExclusiveFilter.SoftObjectPaths.Num() > 0)
		{
			CompiledInclusiveFilter.SoftObjectPaths = CompiledInclusiveFilter.SoftObjectPaths.Difference(CompiledExclusiveFilter.SoftObjectPaths);
			if (CompiledInclusiveFilter.SoftObjectPaths.Num() == 0)
			{
				return false;
			}
			CompiledExclusiveFilter.SoftObjectPaths.Reset();
		}
		if (CompiledInclusiveFilter.ClassPaths.Num() > 0 && CompiledExclusiveFilter.ClassPaths.Num() > 0)
		{
			CompiledInclusiveFilter.ClassPaths = CompiledInclusiveFilter.ClassPaths.Difference(CompiledExclusiveFilter.ClassPaths);
			if (CompiledInclusiveFilter.ClassPaths.Num() == 0 && !!Params.ConvertToUnsupportedAssetDataFilter)
			{
				return false;
			}
			CompiledExclusiveFilter.ClassPaths.Reset();
		}
	}

	// When InPath is a fully virtual folder such as /All, having no package paths is expected
	if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
	{
		// Leave bFilterExcludesAllAssets set to true
		// Otherwise PackagePaths.Num() == 0 is interpreted as everything passes
		return false;
	}

	// If we are enumerating recursively then the inclusive path list will already be fully filtered so just use that
	if (Params.bIncludeFolders && InFilter.bRecursivePaths)
	{
		Params.AssetDataFilter->CachedSubPaths = CompiledInclusiveFilter.PackagePaths;
		for (const FName InternalPath : Params.InternalPaths)
		{
			Params.AssetDataFilter->CachedSubPaths.Remove(InternalPath); // Remove the root as it's not a sub-path
		}
		Params.AssetDataFilter->CachedSubPaths.Sort(FNameLexicalLess()); // Sort as we enumerate these in parent->child order
	}

	// If we got this far then we have something in the filters and need to run the query
	Params.AssetDataFilter->bFilterExcludesAllAssets = false;
	Params.AssetDataFilter->InclusiveFilter = MoveTemp(CompiledInclusiveFilter);
	Params.AssetDataFilter->ExclusiveFilter = MoveTemp(CompiledExclusiveFilter);


	// Compile the filter to show the unsupported items
	if (Params.ConvertToUnsupportedAssetDataFilter)
	{
		if (Params.UnsupportedClassFilter)
		{
			if (const FPathPermissionList* ClassPermissionList = Params.UnsupportedClassFilter->ClassPermissionList.Get())
			{
				if (ClassPermissionList->HasFiltering())
				{
					// Create a Backend filter for the unsupported items
					{
						// Cache the existing class path
						TSet<FTopLevelAssetPath> InclusiveClassPath = MoveTemp(Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths);
						TSet<FTopLevelAssetPath> ExclusiveClassPath = MoveTemp(Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter.ClassPaths);

						// Remove temporally the class filtering from the asset data filter
						TSet<FTopLevelAssetPath> AssetDataInclusiveClassPath = MoveTemp(Params.AssetDataFilter->InclusiveFilter.ClassPaths);
						TSet<FTopLevelAssetPath> AssetDataExclusiveClassPath = MoveTemp(Params.AssetDataFilter->ExclusiveFilter.ClassPaths);

						Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter = Params.AssetDataFilter->InclusiveFilter;
						Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter = Params.AssetDataFilter->ExclusiveFilter;

						// Restore the class filtering
						Params.AssetDataFilter->InclusiveFilter.ClassPaths = MoveTemp(AssetDataInclusiveClassPath);
						Params.AssetDataFilter->ExclusiveFilter.ClassPaths = MoveTemp(AssetDataExclusiveClassPath);

						Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths = MoveTemp(InclusiveClassPath);
						Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter.ClassPaths = MoveTemp(ExclusiveClassPath);
					}

					FPathPermissionList* FolderPermissionList = Params.UnsupportedClassFilter->FolderPermissionList.Get();

					// Compile the inclusive filter for where to show the unsupported asset
					{
						FARCompiledFilter CompiledShowInclusiveFilter;

						// Only show the unsupported asset in the specified folders
						if (FolderPermissionList && FolderPermissionList->HasFiltering())
						{
							FARFilter ShowInclusiveFilter;

							ShowInclusiveFilter.bRecursivePaths = true;

							TArray<FString> AllowList = ClassPermissionList->GetAllowListEntries();
							ShowInclusiveFilter.PackagePaths.Reserve(AllowList.Num());
							for (const FString& Path : AllowList)
							{
								ShowInclusiveFilter.PackagePaths.Emplace(Path);
							}

							Params.AssetRegistry->CompileFilter(ShowInclusiveFilter, CompiledShowInclusiveFilter);
						}

						if (CompiledShowInclusiveFilter.IsEmpty())
						{
							if (Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.IsEmpty())
							{
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.PackagePaths;
							}
							else
							{ 
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.Intersect(Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.PackagePaths);
							}
						}
						else
						{
							CompiledInclusiveFilter.PackagePaths = CompiledShowInclusiveFilter.PackagePaths.Intersect(Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.PackagePaths);

							if (Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.IsEmpty())
							{
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = MoveTemp(CompiledInclusiveFilter.PackagePaths);
							}
							else
							{
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.Intersect(CompiledInclusiveFilter.PackagePaths);
							}
						}
					}

					// Compile the exclusive filter for where to show the unsupported asset
					{
						FARCompiledFilter CompiledShowExclusiveFilter;

						// Only show the unsupported asset in the specified folders
						if (FolderPermissionList && FolderPermissionList->HasFiltering())
						{
							FARFilter ShowExclusiveFilter;

							ShowExclusiveFilter.bRecursivePaths = true;

							TArray<FString> DenyList = ClassPermissionList->GetDenyListEntries();
							ShowExclusiveFilter.PackagePaths.Reserve(DenyList.Num());
							for (const FString& Path : DenyList)
							{
								ShowExclusiveFilter.PackagePaths.Add(FName(Path));
							}

							Params.AssetRegistry->CompileFilter(ShowExclusiveFilter, CompiledShowExclusiveFilter);
						}

						CompiledShowExclusiveFilter.PackagePaths.Append(Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter.PackagePaths);

						if (Params.ConvertToUnsupportedAssetDataFilter->ShowExclusiveFilter.PackagePaths.IsEmpty())
						{
							Params.ConvertToUnsupportedAssetDataFilter->ShowExclusiveFilter.PackagePaths = MoveTemp(CompiledShowExclusiveFilter.PackagePaths);
						}
						else
						{
							Params.ConvertToUnsupportedAssetDataFilter->ShowExclusiveFilter.PackagePaths.Append(MoveTemp(CompiledShowExclusiveFilter.PackagePaths));
						}
					}


					// Compile the convert if fail inclusive filter
					if (ClassPermissionList->HasAllowListEntries())
					{
						FARCompiledFilter CompiledConvertIfFailInclusiveFilter;
						FARFilter ConvertIfFailInclusiveFilter;

						// Remove any inclusive classes that aren't in the explicit allow list set
						TArray<FString> AllowList = ClassPermissionList->GetAllowListEntries();
						ConvertIfFailInclusiveFilter.ClassPaths.Reserve(AllowList.Num());
						for (const FString& Path : AllowList)
						{
							ConvertIfFailInclusiveFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
						}

						Params.AssetRegistry->CompileFilter(ConvertIfFailInclusiveFilter, CompiledConvertIfFailInclusiveFilter);
					

						if (Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths.Num() > 0)
						{
							// Explicit classes given - remove anything not in the allow list class set
							// If the classes resolve as empty then the combined filter will return nothing and can be skipped
							Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths = Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths.Intersect(CompiledConvertIfFailInclusiveFilter.ClassPaths);
						}
						else
						{
							// No explicit classes given - just use the allow list class set
							Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths = MoveTemp(CompiledConvertIfFailInclusiveFilter.ClassPaths);
						}
					}

					// Compile the convert if fail exclusive filter
					if (ClassPermissionList->HasDenyListEntries())
					{
						FARCompiledFilter CompiledConvertIfFailExclusiveFilter;
						FARFilter ConvertIfFailExclusiveFilter;

						// Add any exclusive classes that are in the explicit DenyList set
						TArray<FString> DenyList = ClassPermissionList->GetDenyListEntries();
						ConvertIfFailExclusiveFilter.ClassPaths.Reserve(DenyList.Num());
						for (const FString& Path : DenyList)
						{
							ConvertIfFailExclusiveFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
						}


						Params.AssetRegistry->CompileFilter(ConvertIfFailExclusiveFilter, CompiledConvertIfFailExclusiveFilter);

						Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailExclusiveFilter.ClassPaths.Append(MoveTemp(CompiledConvertIfFailExclusiveFilter.ClassPaths));
					}
				}
			}
		}
	}

	return true;
}

// Note that this function is deprecated and is no longer maintained, see declaration
bool UContentBrowserAssetDataSource::CreateAssetFilter(FAssetFilterInputParams& Params, FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, FCompileARFilterFunc CreateCompiledFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::LegacyCreateAssetFilter);

	// If we're not including files, then we can bail now as the rest of this function deals with assets
	if (!Params.bIncludeFiles)
	{
		return false;
	}

	// If we are filtering all classes, then we can bail now as we won't return any content
	if (Params.ClassPermissionList && Params.ClassPermissionList->IsDenyListAll() && !Params.UnsupportedClassFilter)
	{
		return false;
	}

	// If we are filtering out this path, then we can bail now as it won't return any content
	if (Params.PathPermissionList && !InFilter.bRecursivePaths)
	{
		for (auto It = Params.InternalPaths.CreateIterator(); It; ++It)
		{
			if (!Params.PathPermissionList->PassesStartsWithFilter(*It))
			{
				It.RemoveCurrent();
			}
		}

		if (Params.InternalPaths.Num() == 0)
		{
			return false;
		}
	}

	// Build inclusive asset filter
	FARCompiledFilter CompiledInclusiveFilter;
	{
		// Build the basic inclusive filter from the given data
		{
			FARFilter InclusiveFilter;
			if (Params.ObjectFilter)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InclusiveFilter.ObjectPaths.Append(Params.ObjectFilter->ObjectNamesToInclude);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				InclusiveFilter.TagsAndValues.Append(Params.ObjectFilter->TagsAndValuesToInclude);
				InclusiveFilter.bIncludeOnlyOnDiskAssets |= Params.ObjectFilter->bOnDiskObjectsOnly;
			}
			if (Params.PackageFilter)
			{
				InclusiveFilter.PackageNames.Append(Params.PackageFilter->PackageNamesToInclude);
				InclusiveFilter.PackagePaths.Append(Params.PackageFilter->PackagePathsToInclude);
				InclusiveFilter.bRecursivePaths |= Params.PackageFilter->bRecursivePackagePathsToInclude;
			}
			if (Params.ClassFilter)
			{
				InclusiveFilter.ClassPaths.Append(Params.ClassFilter->ClassNamesToInclude);
				InclusiveFilter.bRecursiveClasses |= Params.ClassFilter->bRecursiveClassNamesToInclude;
			}
			if (Params.CollectionFilter)
			{
				TArray<FSoftObjectPath> ObjectPathsForCollections;
				if (GetObjectPathsForCollections(Params.CollectionFilter->Collections, Params.CollectionFilter->bIncludeChildCollections, nullptr, ObjectPathsForCollections) &&
					ObjectPathsForCollections.Num() == 0)
				{
					// If we had collections but they contained no objects then we can bail as nothing will pass the filter
					return false;
				}
				InclusiveFilter.SoftObjectPaths.Append(MoveTemp(ObjectPathsForCollections));
			}

#if DO_ENSURE
			// Ensure paths do not have trailing slash	
			static const FName RootPath = "/";

			for (const FName ItPath : Params.InternalPaths)
			{
				ensure(ItPath == RootPath || !FStringView(FNameBuilder(ItPath)).EndsWith(TEXT('/')));
			}

			for (const FName ItPath : InclusiveFilter.PackagePaths)
			{
				ensure(ItPath == RootPath || !FStringView(FNameBuilder(ItPath)).EndsWith(TEXT('/')));
			}
#endif // DO_ENSURE

			CreateCompiledFilter(InclusiveFilter, CompiledInclusiveFilter);
		}


		// Add the backend class filtering to the unsupported asset filtering before the class permission are added
		if (Params.ConvertToUnsupportedAssetDataFilter)
		{
			if (Params.UnsupportedClassFilter)
			{
				if (const FPathPermissionList* ClassPermissionList = Params.UnsupportedClassFilter->ClassPermissionList.Get())
				{
					if (ClassPermissionList->HasFiltering())
					{
						if (Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.IsEmpty())
						{
							Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths = CompiledInclusiveFilter.ClassPaths;
						}
						else
						{
							Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths = Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths.Intersect(CompiledInclusiveFilter.ClassPaths);
						}
					}
				}
			}
		}


		// Remove any inclusive paths that aren't under the set of internal paths that we want to enumerate
		{
			FARCompiledFilter CompiledInternalPathFilter;
			{
				FARFilter InternalPathFilter;
				for (const FName InternalPath : Params.InternalPaths)
				{
					InternalPathFilter.PackagePaths.Add(InternalPath);
				}
				InternalPathFilter.bRecursivePaths = InFilter.bRecursivePaths;
				CreateCompiledFilter(InternalPathFilter, CompiledInternalPathFilter);

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::CreateAssetFilter::RemovePaths);
			
					// Remove paths that do not pass item attribute filter (Engine, Plugins, Developer, Localized, __ExternalActors__ etc..)
					for (auto It = CompiledInternalPathFilter.PackagePaths.CreateIterator(); It; ++It)
					{
						FNameBuilder PathStr(*It);
						FStringView Path(PathStr);
						if (!ContentBrowserDataUtils::PathPassesAttributeFilter(Path, 0, InFilter.ItemAttributeFilter))
						{
							It.RemoveCurrent();
						}
					}
				}
			}

			if (CompiledInclusiveFilter.PackagePaths.Num() > 0)
			{
				// Explicit paths given - remove anything not in the internal paths set
				// If the paths resolve as empty then the combined filter will return nothing and can be skipped
				CompiledInclusiveFilter.PackagePaths = CompiledInclusiveFilter.PackagePaths.Intersect(CompiledInternalPathFilter.PackagePaths);
				if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
				{
					return false;
				}
			}
			else
			{
				// No explicit paths given - just use the internal paths set
				CompiledInclusiveFilter.PackagePaths = MoveTemp(CompiledInternalPathFilter.PackagePaths);
			}
		}

		// Add the backend class filtering to the unsupported asset filtering before the class permission are added
		if (Params.ConvertToUnsupportedAssetDataFilter)
		{
			if (Params.UnsupportedClassFilter)
			{
				if (const FPathPermissionList* ClassPermissionList = Params.UnsupportedClassFilter->ClassPermissionList.Get())
				{
					if (ClassPermissionList->HasFiltering())
					{
						Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths.Append(CompiledInclusiveFilter.ClassPaths);
					}
				}
			}
		}

		// Remove any inclusive paths that aren't in the explicit AllowList set
		if (Params.PathPermissionList && Params.PathPermissionList->HasAllowListEntries())
		{
			FARCompiledFilter CompiledPathFilterAllowList;
			{
				FARFilter AllowListPathFilter;
				TArray<FString> AllowList = Params.PathPermissionList->GetAllowListEntries();
				AllowListPathFilter.PackagePaths.Reserve(AllowList.Num());
				for (const FString& Path : AllowList)
				{
					AllowListPathFilter.PackagePaths.Emplace(Path);
				}
				AllowListPathFilter.bRecursivePaths = true;
				CreateCompiledFilter(AllowListPathFilter, CompiledPathFilterAllowList);
			}

			if (CompiledInclusiveFilter.PackagePaths.Num() > 0)
			{
				// Explicit paths given - remove anything not in the allow list paths set
				// If the paths resolve as empty then the combined filter will return nothing and can be skipped
				CompiledInclusiveFilter.PackagePaths = CompiledInclusiveFilter.PackagePaths.Intersect(CompiledPathFilterAllowList.PackagePaths);
				if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
				{
					return false;
				}
			}
			else
			{
				// No explicit paths given - just use the allow list paths set
				CompiledInclusiveFilter.PackagePaths = MoveTemp(CompiledPathFilterAllowList.PackagePaths);
			}
		}

		// Remove any inclusive classes that aren't in the explicit allow list set
		if (Params.ClassPermissionList && Params.ClassPermissionList->HasAllowListEntries())
		{
			FARCompiledFilter CompiledClassFilterAllowList;
			{
				FARFilter AllowListClassFilter;
				TArray<FString> AllowList = Params.ClassPermissionList->GetAllowListEntries();
				for (const FString& Path : AllowList)
				{
					AllowListClassFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
				}
				Params.AssetRegistry->CompileFilter(AllowListClassFilter, CompiledClassFilterAllowList);
			}

			if (CompiledInclusiveFilter.ClassPaths.Num() > 0)
			{
				// Explicit classes given - remove anything not in the allow list class set
				// If the classes resolve as empty then the combined filter will return nothing and can be skipped
				CompiledInclusiveFilter.ClassPaths = CompiledInclusiveFilter.ClassPaths.Intersect(CompiledClassFilterAllowList.ClassPaths);
				if (CompiledInclusiveFilter.ClassPaths.Num() == 0 && !Params.ConvertToUnsupportedAssetDataFilter)
				{
					return false;
				}
			}
			else
			{
				// No explicit classes given - just use the allow list class set
				CompiledInclusiveFilter.ClassPaths = MoveTemp(CompiledClassFilterAllowList.ClassPaths);
			}
		}
	}

	// Build exclusive asset filter
	FARCompiledFilter CompiledExclusiveFilter;
	{
		// Build the basic exclusive filter from the given data
		{
			FARFilter ExclusiveFilter;
			if (Params.ObjectFilter)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ExclusiveFilter.ObjectPaths.Append(Params.ObjectFilter->ObjectNamesToExclude);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				ExclusiveFilter.TagsAndValues.Append(Params.ObjectFilter->TagsAndValuesToExclude);
				ExclusiveFilter.bIncludeOnlyOnDiskAssets |= Params.ObjectFilter->bOnDiskObjectsOnly;
			}
			if (Params.PackageFilter)
			{
				ExclusiveFilter.PackageNames.Append(Params.PackageFilter->PackageNamesToExclude);
				ExclusiveFilter.PackagePaths.Append(Params.PackageFilter->PackagePathsToExclude);
				ExclusiveFilter.bRecursivePaths |= Params.PackageFilter->bRecursivePackagePathsToExclude;
			}
			if (Params.ClassFilter)
			{
				ExclusiveFilter.ClassPaths.Append(Params.ClassFilter->ClassNamesToExclude);
				ExclusiveFilter.bRecursiveClasses |= Params.ClassFilter->bRecursiveClassNamesToExclude;
			}
			CreateCompiledFilter(ExclusiveFilter, CompiledExclusiveFilter);
		}

		// Add any exclusive paths that are in the explicit DenyList set
		if (Params.PathPermissionList && Params.PathPermissionList->HasDenyListEntries())
		{
			FARCompiledFilter CompiledClassFilter;
			{
				FARFilter ClassFilter;
				for (const FString& Path : Params.PathPermissionList->GetDenyListEntries())
				{
					ClassFilter.PackagePaths.Add(FName(Path));
				}
				ClassFilter.bRecursivePaths = true;
				CreateCompiledFilter(ClassFilter, CompiledClassFilter);
			}

			CompiledExclusiveFilter.PackagePaths.Append(CompiledClassFilter.PackagePaths);
		}

		// Add any exclusive classes that are in the explicit DenyList set
		if (Params.ClassPermissionList && Params.ClassPermissionList->HasDenyListEntries())
		{
			FARCompiledFilter CompiledClassFilter;
			{
				FARFilter ClassFilter;
				for (const FString& Path : Params.ClassPermissionList->GetDenyListEntries())
				{
					ClassFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
				}
				Params.AssetRegistry->CompileFilter(ClassFilter, CompiledClassFilter);
			}

			CompiledExclusiveFilter.ClassPaths.Append(CompiledClassFilter.ClassPaths);
		}
	}

	// Apply our exclusive filter to the inclusive one to resolve cases where the exclusive filter cancels out the inclusive filter
	// If any filter components resolve as empty then the combined filter will return nothing and can be skipped
	{
		if (CompiledInclusiveFilter.PackageNames.Num() > 0 && CompiledExclusiveFilter.PackageNames.Num() > 0)
		{
			CompiledInclusiveFilter.PackageNames = CompiledInclusiveFilter.PackageNames.Difference(CompiledExclusiveFilter.PackageNames);
			if (CompiledInclusiveFilter.PackageNames.Num() == 0)
			{
				return false;
			}
			CompiledExclusiveFilter.PackageNames.Reset();
		}
		if (CompiledInclusiveFilter.PackagePaths.Num() > 0 && CompiledExclusiveFilter.PackagePaths.Num() > 0)
		{
			CompiledInclusiveFilter.PackagePaths = CompiledInclusiveFilter.PackagePaths.Difference(CompiledExclusiveFilter.PackagePaths);
			if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
			{
				return false;
			}
			CompiledExclusiveFilter.PackagePaths.Reset();
		}
		if (CompiledInclusiveFilter.SoftObjectPaths.Num() > 0 && CompiledExclusiveFilter.SoftObjectPaths.Num() > 0)
		{
			CompiledInclusiveFilter.SoftObjectPaths = CompiledInclusiveFilter.SoftObjectPaths.Difference(CompiledExclusiveFilter.SoftObjectPaths);
			if (CompiledInclusiveFilter.SoftObjectPaths.Num() == 0)
			{
				return false;
			}
			CompiledExclusiveFilter.SoftObjectPaths.Reset();
		}
		if (CompiledInclusiveFilter.ClassPaths.Num() > 0 && CompiledExclusiveFilter.ClassPaths.Num() > 0)
		{
			CompiledInclusiveFilter.ClassPaths = CompiledInclusiveFilter.ClassPaths.Difference(CompiledExclusiveFilter.ClassPaths);
			if (CompiledInclusiveFilter.ClassPaths.Num() == 0 && !!Params.ConvertToUnsupportedAssetDataFilter)
			{
				return false;
			}
			CompiledExclusiveFilter.ClassPaths.Reset();
		}
	}

	// When InPath is a fully virtual folder such as /All, having no package paths is expected
	if (CompiledInclusiveFilter.PackagePaths.Num() == 0)
	{
		// Leave bFilterExcludesAllAssets set to true
		// Otherwise PackagePaths.Num() == 0 is interpreted as everything passes
		return false;
	}

	// If we are enumerating recursively then the inclusive path list will already be fully filtered so just use that
	if (Params.bIncludeFolders && InFilter.bRecursivePaths)
	{
		Params.AssetDataFilter->CachedSubPaths = CompiledInclusiveFilter.PackagePaths;
		for (const FName InternalPath : Params.InternalPaths)
		{
			Params.AssetDataFilter->CachedSubPaths.Remove(InternalPath); // Remove the root as it's not a sub-path
		}
		Params.AssetDataFilter->CachedSubPaths.Sort(FNameLexicalLess()); // Sort as we enumerate these in parent->child order
	}

	// If we got this far then we have something in the filters and need to run the query
	Params.AssetDataFilter->bFilterExcludesAllAssets = false;
	Params.AssetDataFilter->InclusiveFilter = MoveTemp(CompiledInclusiveFilter);
	Params.AssetDataFilter->ExclusiveFilter = MoveTemp(CompiledExclusiveFilter);


	// Compile the filter to show the unsupported items
	if (Params.ConvertToUnsupportedAssetDataFilter)
	{
		if (Params.UnsupportedClassFilter)
		{
			if (const FPathPermissionList* ClassPermissionList = Params.UnsupportedClassFilter->ClassPermissionList.Get())
			{
				if (ClassPermissionList->HasFiltering())
				{
					// Create a Backend filter for the unsupported items
					{
						// Cache the existing class path
						TSet<FTopLevelAssetPath> InclusiveClassPath = MoveTemp(Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths);
						TSet<FTopLevelAssetPath> ExclusiveClassPath = MoveTemp(Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter.ClassPaths);

						// Remove temporally the class filtering from the asset data filter
						TSet<FTopLevelAssetPath> AssetDataInclusiveClassPath = MoveTemp(Params.AssetDataFilter->InclusiveFilter.ClassPaths);
						TSet<FTopLevelAssetPath> AssetDataExclusiveClassPath = MoveTemp(Params.AssetDataFilter->ExclusiveFilter.ClassPaths);

						Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter = Params.AssetDataFilter->InclusiveFilter;
						Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter = Params.AssetDataFilter->ExclusiveFilter;

						// Restore the class filtering
						Params.AssetDataFilter->InclusiveFilter.ClassPaths = MoveTemp(AssetDataInclusiveClassPath);
						Params.AssetDataFilter->ExclusiveFilter.ClassPaths = MoveTemp(AssetDataExclusiveClassPath);

						Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.ClassPaths = MoveTemp(InclusiveClassPath);
						Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter.ClassPaths = MoveTemp(ExclusiveClassPath);
					}

					FPathPermissionList* FolderPermissionList = Params.UnsupportedClassFilter->FolderPermissionList.Get();

					// Compile the inclusive filter for where to show the unsupported asset
					{
						FARCompiledFilter CompiledShowInclusiveFilter;

						// Only show the unsupported asset in the specified folders
						if (FolderPermissionList && FolderPermissionList->HasFiltering())
						{
							FARFilter ShowInclusiveFilter;

							ShowInclusiveFilter.bRecursivePaths = true;

							TArray<FString> AllowList = ClassPermissionList->GetAllowListEntries();
							ShowInclusiveFilter.PackagePaths.Reserve(AllowList.Num());
							for (const FString& Path : AllowList)
							{
								ShowInclusiveFilter.PackagePaths.Emplace(Path);
							}

							Params.AssetRegistry->CompileFilter(ShowInclusiveFilter, CompiledShowInclusiveFilter);
						}

						if (CompiledShowInclusiveFilter.IsEmpty())
						{
							if (Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.IsEmpty())
							{
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.PackagePaths;
							}
							else
							{ 
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.Intersect(Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.PackagePaths);
							}
						}
						else
						{
							CompiledInclusiveFilter.PackagePaths = CompiledShowInclusiveFilter.PackagePaths.Intersect(Params.ConvertToUnsupportedAssetDataFilter->InclusiveFilter.PackagePaths);

							if (Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.IsEmpty())
							{
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = MoveTemp(CompiledInclusiveFilter.PackagePaths);
							}
							else
							{
								Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths = Params.ConvertToUnsupportedAssetDataFilter->ShowInclusiveFilter.PackagePaths.Intersect(CompiledInclusiveFilter.PackagePaths);
							}
						}
					}

					// Compile the exclusive filter for where to show the unsupported asset
					{
						FARCompiledFilter CompiledShowExclusiveFilter;

						// Only show the unsupported asset in the specified folders
						if (FolderPermissionList && FolderPermissionList->HasFiltering())
						{
							FARFilter ShowExclusiveFilter;

							ShowExclusiveFilter.bRecursivePaths = true;

							TArray<FString> DenyList = ClassPermissionList->GetDenyListEntries();
							ShowExclusiveFilter.PackagePaths.Reserve(DenyList.Num());
							for (const FString& Path : DenyList)
							{
								ShowExclusiveFilter.PackagePaths.Add(FName(Path));
							}

							Params.AssetRegistry->CompileFilter(ShowExclusiveFilter, CompiledShowExclusiveFilter);
						}

						CompiledShowExclusiveFilter.PackagePaths.Append(Params.ConvertToUnsupportedAssetDataFilter->ExclusiveFilter.PackagePaths);

						if (Params.ConvertToUnsupportedAssetDataFilter->ShowExclusiveFilter.PackagePaths.IsEmpty())
						{
							Params.ConvertToUnsupportedAssetDataFilter->ShowExclusiveFilter.PackagePaths = MoveTemp(CompiledShowExclusiveFilter.PackagePaths);
						}
						else
						{
							Params.ConvertToUnsupportedAssetDataFilter->ShowExclusiveFilter.PackagePaths.Append(MoveTemp(CompiledShowExclusiveFilter.PackagePaths));
						}
					}


					// Compile the convert if fail inclusive filter
					if (ClassPermissionList->HasAllowListEntries())
					{
						FARCompiledFilter CompiledConvertIfFailInclusiveFilter;
						FARFilter ConvertIfFailInclusiveFilter;

						// Remove any inclusive classes that aren't in the explicit allow list set
						TArray<FString> AllowList = ClassPermissionList->GetAllowListEntries();
						ConvertIfFailInclusiveFilter.ClassPaths.Reserve(AllowList.Num());
						for (const FString& Path : AllowList)
						{
							ConvertIfFailInclusiveFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
						}

						Params.AssetRegistry->CompileFilter(ConvertIfFailInclusiveFilter, CompiledConvertIfFailInclusiveFilter);
					

						if (Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths.Num() > 0)
						{
							// Explicit classes given - remove anything not in the allow list class set
							// If the classes resolve as empty then the combined filter will return nothing and can be skipped
							Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths = Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths.Intersect(CompiledConvertIfFailInclusiveFilter.ClassPaths);
						}
						else
						{
							// No explicit classes given - just use the allow list class set
							Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter.ClassPaths = MoveTemp(CompiledConvertIfFailInclusiveFilter.ClassPaths);
						}
					}

					// Compile the convert if fail exclusive filter
					if (ClassPermissionList->HasDenyListEntries())
					{
						FARCompiledFilter CompiledConvertIfFailExclusiveFilter;
						FARFilter ConvertIfFailExclusiveFilter;

						// Add any exclusive classes that are in the explicit DenyList set
						TArray<FString> DenyList = ClassPermissionList->GetDenyListEntries();
						ConvertIfFailExclusiveFilter.ClassPaths.Reserve(DenyList.Num());
						for (const FString& Path : DenyList) 
						{
							ConvertIfFailExclusiveFilter.ClassPaths.Add(FTopLevelAssetPath(Path));
						}


						Params.AssetRegistry->CompileFilter(ConvertIfFailExclusiveFilter, CompiledConvertIfFailExclusiveFilter);

						Params.ConvertToUnsupportedAssetDataFilter->ConvertIfFailExclusiveFilter.ClassPaths.Append(MoveTemp(CompiledConvertIfFailExclusiveFilter.ClassPaths));
					}
				}
			}
		}
	}

	return true;
}


void UContentBrowserAssetDataSource::CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::CompileFilter);

	FAssetFilterInputParams Params;
	if (PopulateAssetFilterInputParams(Params, this, AssetRegistry, InFilter, OutCompiledFilter, CollectionManager, &FilterCache))
	{
		const bool bCreatedPathFilter = CreatePathFilter(Params, InPath, InFilter, OutCompiledFilter, [this](FName Path, TFunctionRef<bool(FName)> Callback, bool bRecursive)
		{
			AssetRegistry->EnumerateSubPaths(Path, Callback, bRecursive);
		});

		if (bCreatedPathFilter)
		{
			const bool bWasTemporaryCachingModeEnabled = AssetRegistry->GetTemporaryCachingMode();
			AssetRegistry->SetTemporaryCachingMode(true);
			ON_SCOPE_EXIT
			{
				AssetRegistry->SetTemporaryCachingMode(bWasTemporaryCachingModeEnabled);
			};

			const bool bCreatedAssetFilter = CreateAssetFilter(Params, InPath, InFilter, OutCompiledFilter);

			if (bCreatedAssetFilter)
			{
				// Resolve any custom assets
				if (const FContentBrowserDataLegacyFilter* LegacyFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataLegacyFilter>())
				{
					if (LegacyFilter->OnGetCustomSourceAssets.IsBound())
					{
						FARFilter CustomSourceAssetsFilter;
						CustomSourceAssetsFilter.PackageNames = Params.AssetDataFilter->InclusiveFilter.PackageNames.Array();
						CustomSourceAssetsFilter.PackagePaths = Params.AssetDataFilter->InclusiveFilter.PackagePaths.Array();
						CustomSourceAssetsFilter.ClassPaths = Params.AssetDataFilter->InclusiveFilter.ClassPaths.Array();
						CustomSourceAssetsFilter.TagsAndValues = Params.AssetDataFilter->InclusiveFilter.TagsAndValues;
						CustomSourceAssetsFilter.bIncludeOnlyOnDiskAssets = Params.AssetDataFilter->InclusiveFilter.bIncludeOnlyOnDiskAssets;

						LegacyFilter->OnGetCustomSourceAssets.Execute(CustomSourceAssetsFilter, Params.AssetDataFilter->CustomSourceAssets);
					}
				}
			}
		}
	}
}

enum class EFolderFilterState
{
	None = 0, // Check all filters
	SkipPathInclude = 0x1,
	SkipPathExclude = 0x2,
	SkipPermissionList = 0x3,
};
ENUM_CLASS_FLAGS(EFolderFilterState);

// Possible outcomes of the filtering here
//  Failure - do not visit this path or its children
//	Success - visit this path and it's children
//	Additional info for success - whether we need to check any more path filters - or which ones we still need to check
// return value: success or failure
// InOutFilterState - bitmask of which filters have passed recursively and can be skipped in future
bool PathPassesCompiledDataFilterRecursive(const FContentBrowserCompiledAssetDataFilter& InFilter,
	const FName InInternalPath,
	EFolderFilterState& InOutFilterState)
{
	if (InFilter.ExcludedPackagePaths.Contains(InInternalPath)) // PassesExcludedPathsFilter
	{
		return false;
	}

	FNameBuilder PathStr(InInternalPath);
	FStringView Path(PathStr);
	if (!ContentBrowserDataUtils::PathPassesAttributeFilter(Path, 0, InFilter.ItemAttributeFilter))
	{
		return false;
	}

	auto UpdateFilterState = [&InOutFilterState](EPathPermissionPrefixResult Result, EFolderFilterState Flag) -> bool {
		switch (Result)
		{
			case EPathPermissionPrefixResult::Fail:
			case EPathPermissionPrefixResult::FailRecursive:
				return false;
			case EPathPermissionPrefixResult::PassRecursive:
				InOutFilterState |= Flag;
				return true;
			case EPathPermissionPrefixResult::Pass:
			default:
				return true;
		}
	};

	if (!EnumHasAnyFlags(InOutFilterState, EFolderFilterState::SkipPathInclude))
	{
		if (InFilter.bRecursivePackagePathsToInclude)
		{
			EPathPermissionPrefixResult IncludeResult =
				InFilter.PackagePathsToInclude.PassesStartsWithFilterRecursive(Path);
			if (!UpdateFilterState(IncludeResult, EFolderFilterState::SkipPathInclude))
			{
				return false;
			}
		}
		else
		{
			if (!InFilter.PackagePathsToInclude.PassesFilter(Path))
			{
				return false;
			}
			// No info on recursive pass/fail for exact matches, can't update flags
		}
	}

	if (!EnumHasAnyFlags(InOutFilterState, EFolderFilterState::SkipPathExclude))
	{
		if (InFilter.bRecursivePackagePathsToExclude)
		{
			EPathPermissionPrefixResult ExcludeResult =
				InFilter.PackagePathsToExclude.PassesStartsWithFilterRecursive(Path);
			if (!UpdateFilterState(ExcludeResult, EFolderFilterState::SkipPathExclude))
			{
				return false;
			}
		}
		else
		{
			if (!InFilter.PackagePathsToExclude.PassesFilter(Path))
			{
				return false;
			}
			// No info on recursive pass/fail for exact matches, can't update flags
		}
	}

	if (!EnumHasAnyFlags(InOutFilterState, EFolderFilterState::SkipPermissionList))
	{
		EPathPermissionPrefixResult PermissionResult =
			InFilter.PathPermissionList.PassesStartsWithFilterRecursive(Path, /*bAllowParentPaths*/ true);
		if (!UpdateFilterState(PermissionResult, EFolderFilterState::SkipPermissionList))
		{
			return false;
		}
	}

	return true;
}

void UContentBrowserAssetDataSource::EnumerateFoldersMatchingFilter(
	UContentBrowserDataSource* DataSource, const FContentBrowserCompiledAssetDataFilter* AssetDataFilter, 
	const TGetOrEnumerateSink<FContentBrowserItemData>& InSink, FSubPathEnumerationFunc SubPathEnumeration, FCreateFolderItemFunc CreateFolderItem)
{
	if (AssetDataFilter->bRunFolderQueryOnDemand)
	{
		auto HandleInternalPath = [&DataSource, &InSink, &AssetDataFilter, &SubPathEnumeration, &CreateFolderItem](
									  const FName InInternalPath) {
			TArray<TPair<FName, EFolderFilterState>, TInlineAllocator<16>> PathsToScan;
			PathsToScan.Add({ InInternalPath, EFolderFilterState::None });
			while (PathsToScan.Num() > 0)
			{
				TPair<FName, EFolderFilterState> PathToScan = PathsToScan.Pop(EAllowShrinking::No);
				EFolderFilterState ParentFilterState = PathToScan.Value;
				SubPathEnumeration(
					PathToScan.Key,
					[&DataSource, &InSink, &AssetDataFilter, &PathsToScan, &CreateFolderItem, ParentFilterState](
						FName SubPath) -> bool {
						EFolderFilterState FilterState = ParentFilterState;
						if (PathPassesCompiledDataFilterRecursive(*AssetDataFilter, SubPath, FilterState))
						{
							if (!InSink.ProduceItem(CreateFolderItem(SubPath)))
							{
								return false;
							}

							PathsToScan.Add({ SubPath, FilterState });
						}
						return true;
					},
					false);
			}
		};

		const FName StartingVirtualPath = *AssetDataFilter->VirtualPathToScanOnDemand;
		bool bStartingPathIsFullyVirtual = false;
		DataSource->GetRootPathVirtualTree().PathExists(StartingVirtualPath, bStartingPathIsFullyVirtual);

		if (bStartingPathIsFullyVirtual)
		{
			IAssetRegistry::FPauseBackgroundProcessingScope PauseBackgroundProcessingScope;

			// Virtual paths not supported by PathPassesCompiledDataFilter, enumerate internal paths in hierarchy and propagate results to virtual parents
			TSet<FName> VirtualPathsPassedFilter;
			VirtualPathsPassedFilter.Reserve(DataSource->GetRootPathVirtualTree().NumPaths());
			DataSource->GetRootPathVirtualTree().EnumerateSubPaths(StartingVirtualPath, [&DataSource, &AssetDataFilter, &VirtualPathsPassedFilter](FName VirtualSubPath, FName InternalPath)
			{
				if (!InternalPath.IsNone() && UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(*AssetDataFilter, InternalPath))
				{
					// Propagate result to parents
					for (FName It = VirtualSubPath; !It.IsNone(); It = DataSource->GetRootPathVirtualTree().GetParentPath(It))
					{
						bool bIsAlreadySet = false;
						VirtualPathsPassedFilter.Add(It, &bIsAlreadySet);
						if (bIsAlreadySet)
						{
							break;
						}
					}
				}
				return true;
			}, true);

			// Enumerate virtual path hierarchy again
			TArray<FName, TInlineAllocator<16>> PathsToScan;
			PathsToScan.Add(StartingVirtualPath);
			while (PathsToScan.Num() > 0)
			{
				const FName PathToScan = PathsToScan.Pop(EAllowShrinking::No);
				DataSource->GetRootPathVirtualTree().EnumerateSubPaths(PathToScan, [&DataSource, &InSink, &AssetDataFilter, &VirtualPathsPassedFilter, &PathsToScan, &HandleInternalPath, &CreateFolderItem](FName VirtualSubPath, FName InternalPath)
				{
					if (VirtualPathsPassedFilter.Contains(VirtualSubPath))
					{
						if (!InternalPath.IsNone())
						{
							if (!InSink.ProduceItem(CreateFolderItem(InternalPath)))
							{
								return false;
							}

							HandleInternalPath(InternalPath);
						}
						else
						{
							if (!InSink.ProduceItem(DataSource->CreateVirtualFolderItem(VirtualSubPath)))
							{
								return false;
							}
						}

						PathsToScan.Add(VirtualSubPath);
					}
					return true;
				}, false);
			}
		}
		else
		{
			FName InternalPath;
			if (DataSource->TryConvertVirtualPathToInternal(StartingVirtualPath, InternalPath))
			{
				HandleInternalPath(InternalPath);
			}
		}
	}
	else
	{
		for (const FName& SubPath : AssetDataFilter->CachedSubPaths)
		{
			if (!InSink.ProduceItem(CreateFolderItem(SubPath)))
			{
				return;
			}
		}
	}
}

void UContentBrowserAssetDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	EnumerateItemsMatchingFilter(InFilter, TGetOrEnumerateSink<FContentBrowserItemData>(MoveTemp(InCallback)));
}


void UContentBrowserAssetDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return;
	}
	
	const FContentBrowserCompiledAssetDataFilter* AssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledAssetDataFilter>();
	if (!AssetDataFilter)
	{
		return;
	}

	if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		auto EnumerateSubPaths = [this](FName Path, TFunctionRef<bool(FName)> Callback, bool bRecursive)
		{
			AssetRegistry->EnumerateSubPaths(Path, Callback, bRecursive);
		};
		auto CreateFolderItem = [this](FName Path) -> FContentBrowserItemData
		{
			return CreateAssetFolderItem(Path);
		};
		EnumerateFoldersMatchingFilter(this, AssetDataFilter, InSink, EnumerateSubPaths, CreateFolderItem);
	}

	if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !AssetDataFilter->bFilterExcludesAllAssets)
	{
		for (const FAssetData& CustomSourceAsset : AssetDataFilter->CustomSourceAssets)
		{
			if (!InSink.ProduceItem(CreateAssetFileItem(CustomSourceAsset)))
			{
				return;
			}
		}

		if (const FContentBrowserCompiledUnsupportedAssetDataFilter* UnsupportedAssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledUnsupportedAssetDataFilter>())
		{
			const TSharedPtr<FPathPermissionList>& ShowPrivateContentPermissionList = IContentBrowserSingleton::Get().GetShowPrivateContentPermissionList();
			const TSharedPtr<FPathPermissionList>& ShowEpicInternalContentPermissionList = IContentBrowserSingleton::Get().GetShowEpicInternalContentPermissionList();

			// Using the show unsupported asset filter
			AssetRegistry->EnumerateAssets(UnsupportedAssetDataFilter->InclusiveFilter, [this, &InSink, &AssetDataFilter, UnsupportedAssetDataFilter, &ShowPrivateContentPermissionList, &ShowEpicInternalContentPermissionList](const FAssetData& AssetData)
			{
				if (ContentBrowserAssetData::IsPrimaryAsset(AssetData) && AssetData.GetOptionalOuterPathName().IsNone())
				{
					const bool bPassesExclusiveFilter = UnsupportedAssetDataFilter->ExclusiveFilter.IsEmpty() || !AssetRegistry->IsAssetIncludedByFilter(AssetData, UnsupportedAssetDataFilter->ExclusiveFilter);
					if (bPassesExclusiveFilter)
					{
						// Exclude private assets that do not pass ShowPrivateContentPermissionList
						if (AssetData.GetAssetAccessSpecifier() == EAssetAccessSpecifier::Private)
						{
							if (!ShowPrivateContentPermissionList->PassesStartsWithFilter(FNameBuilder(AssetData.PackageName)))
							{
								return true;
							}
						}

						// Exclude Epic internal assets that do not pass ShowEpicInternalContentPermissionList or ShowPrivateContentPermissionList
						if (AssetData.GetAssetAccessSpecifier() == EAssetAccessSpecifier::EpicInternal)
						{
							FNameBuilder PackageNameBuilder(AssetData.PackageName);
							if (!(ShowEpicInternalContentPermissionList->PassesStartsWithFilter(PackageNameBuilder) || ShowPrivateContentPermissionList->PassesStartsWithFilter(PackageNameBuilder)))
							{
								return true;
							}
						}

						// Should this asset be presented as unsupported
						if (!(AssetRegistry->IsAssetIncludedByFilter(AssetData, UnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter) && (UnsupportedAssetDataFilter->ConvertIfFailExclusiveFilter.IsEmpty() || AssetRegistry->IsAssetExcludedByFilter(AssetData, UnsupportedAssetDataFilter->ConvertIfFailExclusiveFilter))) // Do we fail the supported filter?
							&& (AssetRegistry->IsAssetIncludedByFilter(AssetData, UnsupportedAssetDataFilter->ShowInclusiveFilter) && (UnsupportedAssetDataFilter->ShowExclusiveFilter.IsEmpty() || AssetRegistry->IsAssetExcludedByFilter(AssetData, UnsupportedAssetDataFilter->ShowExclusiveFilter)))) // Do we pass the show filter for the unsupported asset?
						{
							return InSink.ProduceItem(CreateUnsupportedAssetFileItem(AssetData));
						}

						// Normal item test it against the class filter
						if ((AssetDataFilter->InclusiveFilter.ClassPaths.IsEmpty() || AssetDataFilter->InclusiveFilter.ClassPaths.Contains(AssetData.AssetClassPath)) && !AssetDataFilter->ExclusiveFilter.ClassPaths.Contains(AssetData.AssetClassPath))
						{
							return InSink.ProduceItem(CreateAssetFileItem(AssetData));
						}
					}
				}
				return true;
			});
			return;
		}

		auto ProduceAssets = [&AssetDataFilter, &InSink, this](TArray<FAssetData>& Assets, const TSet<FName>& IgnorePackageNames) {
			InSink.ReserveMore(Assets.Num());
			
			FAssetPropertyTagCache& TagCache = FAssetPropertyTagCache::Get();
			for (const FAssetData& AssetData : Assets)
			{
				TagCache.TryCacheClass(AssetData.AssetClassPath);
			}
			
			for (FAssetData& AssetData : Assets)
			{
				if (IgnorePackageNames.Contains(AssetData.PackageName))
				{
					AssetData = FAssetData{};
				}
			}

			if (!AssetDataFilter->ExclusiveFilter.IsEmpty())
			{
				for (FAssetData& AssetData : Assets)
				{
					if (AssetRegistry->IsAssetIncludedByFilter(AssetData, AssetDataFilter->ExclusiveFilter))
					{
						AssetData = FAssetData{};
					}
				}
			}
			
			// For batches above some arbitrary threshold, run conversion in parallel
			if (AssetDataSource::bAllowInternalParallelism && Assets.Num() > 1024 * 16)
			{
				TArray<FContentBrowserItemData> Converted;
				Converted.Reserve(Assets.Num());
				Converted.AddUninitialized(Assets.Num());
				ParallelFor(TEXT("ConvertAssetsToContentBrowserItems"), Assets.Num(), 1024 * 16, [&Assets, &Converted, this](int32 Index) {
					if (Assets[Index].IsValid() && ContentBrowserAssetData::IsPrimaryAsset(Assets[Index]))
					{
						new (&Converted[Index]) FContentBrowserItemData(CreateAssetFileItem(MoveTemp(Assets[Index])));
					}
					else
					{
						new (&Converted[Index]) FContentBrowserItemData();
					}
				});
				for (FContentBrowserItemData& Item : Converted)
				{
					if (Item.IsValid())
					{
						InSink.ProduceItem(MoveTemp(Item));
					}
				}
			}
			else
			{
				for (FAssetData& AssetData : Assets)
				{
					if (AssetData.IsValid() && ContentBrowserAssetData::IsPrimaryAsset(AssetData))
					{
						InSink.ProduceItem(CreateAssetFileItem(MoveTemp(AssetData)));
					}
				}
			}
		};

		UE::Tasks::TTask<TArray<FAssetData>> DiskTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [AssetRegistry=AssetRegistry, &AssetDataFilter]() {
			TArray<FAssetData> Assets;
			FARCompiledFilter OnDiskFilter = AssetDataFilter->InclusiveFilter;
			OnDiskFilter.bIncludeOnlyOnDiskAssets = true;
			AssetRegistry->GetAssets(OnDiskFilter, Assets);
			return MoveTemp(Assets);
		});

		TSet<FName> IgnorePackages;
		if (AssetDataFilter->InclusiveFilter.PackageNames.Num() != 0 || !AssetDataSource::bOptimizeEnumerateInMemoryAssets) 
		{
			TArray<FAssetData> InMemoryAssets;
			AssetRegistry->GetInMemoryAssets(AssetDataFilter->InclusiveFilter, InMemoryAssets);
			ProduceAssets(InMemoryAssets, IgnorePackages);
			Algo::Transform(InMemoryAssets, IgnorePackages, [](const FAssetData& AssetData) { return AssetData.PackageName; });
		}
		else
		{
			TArray<FAssetData> InMemoryAssets;
			FARCompiledFilter InMemoryFilter = AssetDataFilter->InclusiveFilter;
			ForEachObjectOfClass(UPackage::StaticClass(), [&InMemoryFilter](UObject* Object)
			{
				UPackage* Package = CastChecked<UPackage>(Object);
				if (Package->HasAnyFlags(RF_ClassDefaultObject))
				{
					return;
				}

				if (Package->IsDirty() || Package->HasAnyPackageFlags(PKG_NewlyCreated))
				{
					InMemoryFilter.PackageNames.Add(Package->GetFName());
				}
			});
			if (InMemoryFilter.PackageNames.Num() > 0)
			{
				AssetRegistry->GetInMemoryAssets(InMemoryFilter, InMemoryAssets);

				ProduceAssets(InMemoryAssets, IgnorePackages);
				Algo::Transform(InMemoryAssets, IgnorePackages, [](const FAssetData& AssetData) { return AssetData.PackageName; });
			}
		}

		DiskTask.Wait();
		TArray<FAssetData> DiskAssets = MoveTemp(DiskTask.GetResult());
		ProduceAssets(DiskAssets, IgnorePackages);
	}
}

void UContentBrowserAssetDataSource::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return;
	}
	
	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		if (AssetRegistry->PathExists(InternalPath))
		{
			InCallback(CreateAssetFolderItem(InternalPath));
		}
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
	{
		FARFilter ARFilter;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ARFilter.ObjectPaths.Add(InternalPath);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		AssetRegistry->EnumerateAssets(ARFilter, [this, &InCallback](const FAssetData& AssetData)
		{
			if (ContentBrowserAssetData::IsPrimaryAsset(AssetData))
			{
				return InCallback(CreateAssetFileItem(AssetData));
			}
			return true;
		});
	}
}

bool UContentBrowserAssetDataSource::EnumerateItemsAtPaths(const TArrayView<FContentBrowserItemPath> InPaths, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		for (const FContentBrowserItemPath& InPath : InPaths)
		{
			if (InPath.HasInternalPath())
			{
				if (AssetRegistry->PathExists(InPath.GetInternalPathName()))
				{
					if (!InCallback(CreateAssetFolderItem(InPath.GetInternalPathName())))
					{
						return false;
					}
				}
			}
		}
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !InPaths.IsEmpty())
	{
		FARFilter ARFilter;
		
		// TODO: EnumerateAssets for in memory assets needs optimization, currently enumerates every UObject in memory instead of calling find
		ARFilter.bIncludeOnlyOnDiskAssets = true;

		for (const FContentBrowserItemPath& InPath : InPaths)
		{
			if (InPath.HasInternalPath())
			{
				ARFilter.PackageNames.Add(InPath.GetInternalPathName());
			}
		}

		auto FileFoundCallback = [this, &InCallback](const FAssetData& AssetData)
		{
			if (ContentBrowserAssetData::IsPrimaryAsset(AssetData))
			{
				return InCallback(CreateAssetFileItem(AssetData));
			}
			return true;
		};

		if (!AssetRegistry->EnumerateAssets(ARFilter, FileFoundCallback))
		{
			return false;
		}
	}

	return true;
}

void UContentBrowserAssetDataSource::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		// Could be a package path.
		if (ensure(AssetRegistry))
		{
			FStringView Path = InPath;

			// Internal paths have no trailing /.
			while (FPathViews::HasRedundantTerminatingSeparator(Path))
			{
				Path.LeftChopInline(1);
			}

			FName InternalFolderPath(Path, FNAME_Find);
			if (!InternalFolderPath.IsNone() && AssetRegistry->PathExists(InternalFolderPath))
			{
				InCallback(CreateAssetFolderItem(InternalFolderPath));
			}
		}
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
	{
		// Could be an object path.
		if (ensure(AssetRegistry))
		{
			FSoftObjectPath ObjectPath(FPackageName::ExportTextPathToObjectPath(InPath));
			if (ObjectPath.IsValid())
			{
				FAssetData AssetData = AssetRegistry->GetAssetByObjectPath(ObjectPath);
				if (AssetData.IsValid())
				{
					if (ContentBrowserAssetData::IsPrimaryAsset(AssetData))
					{
						InCallback(CreateAssetFileItem(AssetData));
					}

					return;
				}
			}
		}

		// Could be a package name or file name.
		if (ensure(AssetRegistry))
		{
			FName PackageName;
			if (FPackageName::IsValidLongPackageName(InPath))
			{
				PackageName = FName(InPath, FNAME_Find);
			}
			else
			{
				FNameBuilder PackageNameBuilder;
				if (FPackageName::TryConvertFilenameToLongPackageName(InPath, PackageNameBuilder))
				{
					PackageName = FName(PackageNameBuilder, FNAME_Find);
				}
			}

			if (!PackageName.IsNone())
			{
				TArray<FAssetData> Assets;
				if (AssetRegistry->GetAssetsByPackageName(PackageName, Assets))
				{
					for (const FAssetData& AssetData : Assets)
					{
						if (ContentBrowserAssetData::IsPrimaryAsset(AssetData))
						{
							InCallback(CreateAssetFileItem(AssetData));
						}
					}
				}
			}
		}
	}
}

bool UContentBrowserAssetDataSource::IsDiscoveringItems(FText* OutStatus)
{
	if (AssetRegistry->IsLoadingAssets())
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutStatus, DiscoveryStatusText);
		return true;
	}

	return false;
}

bool UContentBrowserAssetDataSource::PrioritizeSearchPath(const FName InPath)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return false;
	}

	AssetRegistry->PrioritizeSearchPath(InternalPath.ToString() / FString());
	return true;
}

bool UContentBrowserAssetDataSource::IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& InContentsFilter)
{
	const TSharedPtr<FPathPermissionList>& ShowPrivateContentPermissionList = IContentBrowserSingleton::Get().GetShowPrivateContentPermissionList();

	auto IsInternalFolderVisible = [this, &InContentsFilter, &ShowPrivateContentPermissionList](FName InternalFolderPath) {
		const EContentBrowserFolderAttributes FolderAttributes = GetAssetFolderAttributes(InternalFolderPath);
		if (EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::AlwaysVisible))
		{
			return true;
		}

		// Hide folders that only contain cooked private content
		if (EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::HasAssets | EContentBrowserFolderAttributes::HasRedirectors))
		{
			if (!EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::HasVisiblePublicContent))
			{
				FNameBuilder InternalFolderPathBuilder(InternalFolderPath);
				if (!ShowPrivateContentPermissionList->PassesStartsWithFilter(InternalFolderPathBuilder))
				{
					return false;
				}
			}
		}

		if (EnumHasAnyFlags(InContentsFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeAssets)
			&& EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::HasAssets))
		{
			return true;
		}
		if (EnumHasAnyFlags(InContentsFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeRedirectors)
			&& EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::HasRedirectors))
		{
			return true;
		}
		if (InContentsFilter.HideFolderIfEmptyFilter)
		{
			FNameBuilder InternalFolderPathBuilder(InternalFolderPath);
			return !HideFolderIfEmpty(*InContentsFilter.HideFolderIfEmptyFilter, InternalFolderPath, InternalFolderPathBuilder);
		}
		return false;
	};

	FName ConvertedPath;
	const EContentBrowserPathType ConvertedPathType = TryConvertVirtualPath(InPath, ConvertedPath);
	if (ConvertedPathType == EContentBrowserPathType::Internal)
	{
		if (!IsKnownContentPath(ConvertedPath))
		{
			return false;
		}
		return IsInternalFolderVisible(ConvertedPath);
	}
	else if (ConvertedPathType == EContentBrowserPathType::Virtual)
	{
		bool bAnyVisible = false;
		// Make virtual folders visible if any of their child folders will be visible
		RootPathVirtualTree.EnumerateSubPaths(
			ConvertedPath,
			[this, &bAnyVisible, &IsInternalFolderVisible](FName ChildVirtualPath, FName ChildInternalPath) -> bool {
				if (!ChildInternalPath.IsNone())
				{
					bAnyVisible = IsInternalFolderVisible(ChildInternalPath);
				}
				return !bAnyVisible;
			},
			true);
		return true;
	}
	else
	{
		return false;
	}
}

bool UContentBrowserAssetDataSource::CanCreateFolder(const FName InPath, FText* OutErrorMsg)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return false;
	}

	if (!IsKnownContentPath(InternalPath))
	{
		return false;
	}

	return ContentBrowserAssetData::CanModifyPath(AssetTools, InternalPath, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter, FContentBrowserItemDataTemporaryContext& OutPendingItem)
{
	const FString ParentPath = FPackageName::GetLongPackagePath(InPath.ToString());
	FName InternalParentPath;
	if (!TryConvertVirtualPathToInternal(*ParentPath, InternalParentPath))
	{
		return false;
	}

	const FString FolderItemName = FPackageName::GetShortName(InPath);
	FString InternalPathString = InternalParentPath.ToString() + TEXT("/") + FolderItemName;

	FContentBrowserItemData NewItemData(
		this,
		EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Asset | EContentBrowserItemFlags::Temporary_Creation,
		InPath,
		*FolderItemName,
		FText::AsCultureInvariant(FolderItemName),
		MakeShared<FContentBrowserAssetFolderItemDataPayload>(*InternalPathString),
		FName(InternalPathString)
		);

	OutPendingItem = FContentBrowserItemDataTemporaryContext(
		MoveTemp(NewItemData),
		FContentBrowserItemDataTemporaryContext::FOnValidateItem::CreateLambda([DataSource = TWeakObjectPtr<UContentBrowserAssetDataSource>(this), HideFolderIfEmptyFilter](const FContentBrowserItemData& InItem, const FString& InProposedName, FText* OutErrorMsg)
			{
				if (UContentBrowserAssetDataSource* DataSourcePtr = DataSource.Get())
				{
					return DataSourcePtr->CanRenameItem(InItem, &InProposedName, HideFolderIfEmptyFilter.Get(), OutErrorMsg);
				}
				return false;
			}),
		FContentBrowserItemDataTemporaryContext::FOnFinalizeItem::CreateUObject(this, &UContentBrowserAssetDataSource::OnFinalizeCreateFolder)
		);

	return true;
}

bool UContentBrowserAssetDataSource::DoesItemPassFolderFilter(UContentBrowserDataSource* DataSource, const FContentBrowserItemData& InItem, const FContentBrowserCompiledAssetDataFilter& Filter)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = ContentBrowserAssetData::GetAssetFolderItemPayload(DataSource, InItem))
	{
		if (Filter.bRunFolderQueryOnDemand)
		{
			bool bIsUnderSearchPath = false;
			const FString& PathToScan = Filter.VirtualPathToScanOnDemand;
			if (PathToScan == TEXT("/"))
			{
				bIsUnderSearchPath = true;
			}
			else 
			{
				const FName VirtualPath = InItem.GetVirtualPath();
				FNameBuilder VirtualPathBuilder(VirtualPath);
				const FStringView VirtualPathView(VirtualPathBuilder);
				if (VirtualPathView.StartsWith(PathToScan))
				{
					if ((VirtualPathView.Len() <= PathToScan.Len()) || (VirtualPathView[PathToScan.Len()] == TEXT('/')))
					{
						bIsUnderSearchPath = true;
					}
				}
			}

			const bool bPassesCompiledFilter = bIsUnderSearchPath && UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(Filter, FolderPayload->GetInternalPath());

			return bIsUnderSearchPath && bPassesCompiledFilter;
		}
		else
		{
			return Filter.CachedSubPaths.Contains(FolderPayload->GetInternalPath());
		}
	}
	else
	{
		bool bPasses = false;
		DataSource->GetRootPathVirtualTree().EnumerateSubPaths(InItem.GetVirtualPath(), [&bPasses, &Filter](FName VirtualSubPath, FName InternalPath)
		{
			if (!InternalPath.IsNone())
			{
				if (UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(Filter, InternalPath))
				{
					bPasses = true;

					// Stop enumerate
					return false;
				}
			}
			return true;
		}, true);

		return bPasses;
	}
}

bool UContentBrowserAssetDataSource::DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return false;
	}

	const FContentBrowserCompiledAssetDataFilter* AssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledAssetDataFilter>();
	if (!AssetDataFilter)
	{
		return false;
	}

	switch (InItem.GetItemType())
	{
	case EContentBrowserItemFlags::Type_Folder:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
		{
			return DoesItemPassFolderFilter(this, InItem, *AssetDataFilter);
		}
		break;

	case EContentBrowserItemFlags::Type_File:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !AssetDataFilter->bFilterExcludesAllAssets)
		{
			auto FilterWithAssetData = [this, AssetDataFilter](const FAssetData& InAssetData, const FARCompiledFilter& InclusiveFilter, const FARCompiledFilter& ExclusiveFilter) -> bool
				{
				// Must pass Inclusive AND !Exclusive, or be a CustomAsset
				return (AssetRegistry->IsAssetIncludedByFilter(InAssetData, InclusiveFilter) // InclusiveFilter
					&& (ExclusiveFilter.IsEmpty() || !AssetRegistry->IsAssetIncludedByFilter(InAssetData, ExclusiveFilter))) // ExclusiveFilter
					|| AssetDataFilter->CustomSourceAssets.Contains(InAssetData); // CustomAsset
				};

			if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InItem))
			{
				return FilterWithAssetData(AssetPayload->GetAssetData(), AssetDataFilter->InclusiveFilter, AssetDataFilter->ExclusiveFilter);
			}

			if (TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UnsupportedAssetPayload = GetUnsupportedAssetFileItemPayload(InItem))
			{
				if (const FContentBrowserCompiledUnsupportedAssetDataFilter* UnsupportedAssetFilter = FilterList->FindFilter<FContentBrowserCompiledUnsupportedAssetDataFilter>())
				{
					if (const FAssetData* AssetData = UnsupportedAssetPayload->GetAssetDataIfAvailable())
					{
						return FilterWithAssetData(*AssetData, UnsupportedAssetFilter->InclusiveFilter, UnsupportedAssetFilter->ExclusiveFilter);
					}
				}
			}
		}
		break;

	default:
		break;
	}
	
	return false;
}

bool UContentBrowserAssetDataSource::ConvertItemForFilter(FContentBrowserItemData& Item, const FContentBrowserDataCompiledFilter& InFilter)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return false;
	}

	const FContentBrowserCompiledUnsupportedAssetDataFilter* UnsupportedAssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledUnsupportedAssetDataFilter>();
	if (!UnsupportedAssetDataFilter)
	{
		return false;
	}

	if (Item.GetOwnerDataSource() != this)
	{
		return false;
	}

	const FContentBrowserCompiledAssetDataFilter* AssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledAssetDataFilter>();

	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(Item))
	{
		const FAssetData& AssetData = AssetPayload->GetAssetData();
		if (AssetData.GetOptionalOuterPathName().IsNone() && ContentBrowserAssetData::IsPrimaryAsset(AssetData))
		{
			if (!AssetDataFilter || !AssetDataFilter->CustomSourceAssets.Contains(AssetData))
			{
				if (!(AssetRegistry->IsAssetIncludedByFilter(AssetData, UnsupportedAssetDataFilter->ConvertIfFailInclusiveFilter) && (UnsupportedAssetDataFilter->ConvertIfFailExclusiveFilter.IsEmpty() || AssetRegistry->IsAssetExcludedByFilter(AssetData, UnsupportedAssetDataFilter->ConvertIfFailExclusiveFilter))) // Do we fail the supported filter?
					&& (AssetRegistry->IsAssetIncludedByFilter(AssetData, UnsupportedAssetDataFilter->ShowInclusiveFilter) && (UnsupportedAssetDataFilter->ShowExclusiveFilter.IsEmpty() || AssetRegistry->IsAssetExcludedByFilter(AssetData, UnsupportedAssetDataFilter->ShowExclusiveFilter)))) // Do we pass the show filter for the unsupported asset?
				{
					const EAssetAccessSpecifier AssetAccessSpecifier = AssetData.GetAssetAccessSpecifier();
					if (AssetAccessSpecifier == EAssetAccessSpecifier::Private)
					{
						// Exclude private assets that do not pass ShowPrivateContentPermissionList
						if (!IContentBrowserSingleton::Get().GetShowPrivateContentPermissionList()->PassesStartsWithFilter(FNameBuilder(AssetData.PackageName)))
						{
							return false;
						}
					}
					else if (AssetAccessSpecifier == EAssetAccessSpecifier::EpicInternal)
					{
						// Exclude EpicInternal assets that do not pass ShowEpicInternalContentPermissionList
						if (!IContentBrowserSingleton::Get().GetShowEpicInternalContentPermissionList()->PassesStartsWithFilter(FNameBuilder(AssetData.PackageName)))
						{
							return false;
						}
					}

					Item = CreateUnsupportedAssetFileItem(AssetData);
					return true;
				}
			}
		}
	}

	return false;
}

bool UContentBrowserAssetDataSource::GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	return ContentBrowserAssetData::GetItemAttribute(ContentBrowserModule, this, InItem, InIncludeMetaData, InAttributeKey, OutAttributeValue);
}

bool UContentBrowserAssetDataSource::GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	return ContentBrowserAssetData::GetItemAttributes(this, InItem, InIncludeMetaData, OutAttributeValues);
}

bool UContentBrowserAssetDataSource::GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	return ContentBrowserAssetData::GetItemPhysicalPath(this, InItem, OutDiskPath);
}

bool UContentBrowserAssetDataSource::GetItemAssetAccessSpecifier(const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier)
{
	return ContentBrowserAssetData::GetAssetAccessSpecifier(this, InItem, OutAssetAccessSpecifier);
}

bool UContentBrowserAssetDataSource::CanModifyItemAssetAccessSpecifier(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::CanModifyAssetAccessSpecifier(AssetTools, this, InItem);
}

bool UContentBrowserAssetDataSource::IsItemDirty(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::IsItemDirty(this, InItem);
}

bool UContentBrowserAssetDataSource::CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanEditItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::EditItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::EditItems(AssetTools, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAssetDataSource::BulkEditItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::EditItems(AssetTools, this, InItems);
}

bool UContentBrowserAssetDataSource::CanViewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanViewItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::ViewItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::ViewItems(AssetTools, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAssetDataSource::BulkViewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::ViewItems(AssetTools, this, InItems);
}

bool UContentBrowserAssetDataSource::CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanPreviewItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::PreviewItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::PreviewItems(AssetTools, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAssetDataSource::BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::PreviewItems(AssetTools, this, InItems);
}

bool UContentBrowserAssetDataSource::CanDuplicateItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanDuplicateItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::DuplicateItem(const FContentBrowserItemData& InItem, FContentBrowserItemDataTemporaryContext& OutPendingItem)
{
	UObject* SourceAsset = nullptr;
	FAssetData NewAssetData;
	if (ContentBrowserAssetData::DuplicateItem(AssetTools, this, InItem, SourceAsset, NewAssetData))
	{
		FName VirtualizedPath;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FName InternalPath = NewAssetData.ObjectPath;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		TryConvertInternalPathToVirtual(InternalPath, VirtualizedPath);

		FContentBrowserItemData NewItemData(
			this,
			EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Asset
				| EContentBrowserItemFlags::Temporary_Duplication,
			VirtualizedPath,
			NewAssetData.AssetName,
			FText::AsCultureInvariant(NewAssetData.AssetName.ToString()),
			MakeShared<FContentBrowserAssetFileItemDataPayload_Duplication>(MoveTemp(NewAssetData), SourceAsset),
			{ InternalPath });

		OutPendingItem = FContentBrowserItemDataTemporaryContext(
			MoveTemp(NewItemData),
			FContentBrowserItemDataTemporaryContext::FOnValidateItem::CreateUObject(
				this, &UContentBrowserAssetDataSource::OnValidateItemName),
			FContentBrowserItemDataTemporaryContext::FOnFinalizeItem::CreateUObject(
				this, &UContentBrowserAssetDataSource::OnFinalizeDuplicateAsset));

		return true;
	}

	return false;
}

bool UContentBrowserAssetDataSource::BulkDuplicateItems(TArrayView<const FContentBrowserItemData> InItems, TArray<FContentBrowserItemData>& OutNewItems)
{
	TArray<FAssetData> NewAssets;
	if (ContentBrowserAssetData::DuplicateItems(AssetTools, this, InItems, NewAssets))
	{
		for (const FAssetData& NewAsset : NewAssets)
		{
			OutNewItems.Emplace(CreateAssetFileItem(NewAsset));
		}

		return true;
	}

	return false;
}

bool UContentBrowserAssetDataSource::CanSaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanSaveItem(AssetTools, this, InItem, InSaveFlags, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::SaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags)
{
	return ContentBrowserAssetData::SaveItems(AssetTools, this, MakeArrayView(&InItem, 1), InSaveFlags);
}

bool UContentBrowserAssetDataSource::BulkSaveItems(TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags)
{
	return ContentBrowserAssetData::SaveItems(AssetTools, this, InItems, InSaveFlags);
}

bool UContentBrowserAssetDataSource::CanDeleteItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanDeleteItem(AssetTools, AssetRegistry, this, InItem, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::DeleteItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::DeleteItems(ContentBrowserModule, AssetTools, AssetRegistry, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAssetDataSource::BulkDeleteItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::DeleteItems(ContentBrowserModule, AssetTools, AssetRegistry, this, InItems);
}

bool UContentBrowserAssetDataSource::CanPrivatizeItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanPrivatizeItem(AssetTools, AssetRegistry, this, InItem, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::PrivatizeItem(const FContentBrowserItemData& InItem, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	return ContentBrowserAssetData::PrivatizeItems(AssetTools, AssetRegistry, this, MakeArrayView(&InItem, 1), InAssetAccessSpecifier);
}

bool UContentBrowserAssetDataSource::BulkPrivatizeItems(TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	return ContentBrowserAssetData::PrivatizeItems(AssetTools, AssetRegistry, this, InItems, InAssetAccessSpecifier);
}

bool UContentBrowserAssetDataSource::CanRenameItem(const FContentBrowserItemData& InItem, const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg)
{
	bool bCheckUniqueName = true;

	if (InNewName != nullptr && HideFolderIfEmptyFilter != nullptr)
	{
		if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InItem))
		{
			FString NewInternalPathString = FPaths::GetPath(FolderPayload->GetInternalPath().ToString()) / *InNewName;
			if (NewInternalPathString.Len() < NAME_SIZE) // If the path is too long, CanRenameItem will fail anyways, let it produce the error message.
			{
				FName NewInternalPath(NewInternalPathString);

				const EContentBrowserFolderAttributes FolderAttributes = GetAssetFolderAttributes(NewInternalPath);
				if (!EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::AlwaysVisible | EContentBrowserFolderAttributes::HasAssets | EContentBrowserFolderAttributes::HasRedirectors) &&
					HideFolderIfEmpty(*HideFolderIfEmptyFilter, NewInternalPath, NewInternalPathString))
				{
					// We are renaming to an existing hidden folder name, disable the unique name check since it will fail.
					bCheckUniqueName = false;
				}
			}
		}
	}

	return ContentBrowserAssetData::CanRenameItem(AssetTools, this, InItem, bCheckUniqueName, InNewName, OutErrorMsg);
}

bool UContentBrowserAssetDataSource::RenameItem(const FContentBrowserItemData& InItem, const FString& InNewName, FContentBrowserItemData& OutNewItem)
{
	if (ContentBrowserAssetData::RenameItem(AssetTools, AssetRegistry, this, InItem, InNewName))
	{
		switch (InItem.GetItemType())
		{
		case EContentBrowserItemFlags::Type_Folder:
			if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InItem))
			{
				const FString NewFolderPath = FPaths::GetPath(FolderPayload->GetInternalPath().ToString()) / InNewName;
				AssetViewUtils::OnAlwaysShowPath().Broadcast(NewFolderPath);
				OutNewItem = CreateAssetFolderItem(FName(NewFolderPath));
			}
			break;

		case EContentBrowserItemFlags::Type_File:
			if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InItem))
			{
				// The asset should already be loaded from preforming the rename
				// We can use the renamed object instance to create the new asset data for the renamed item
				if (UObject* Asset = AssetPayload->GetAsset())
				{
					OutNewItem = CreateAssetFileItem(FAssetData(Asset));
				}
			}
			break;

		default:
			break;
		}

		return true;
	}

	return false;
}

bool UContentBrowserAssetDataSource::CanCopyItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg)
{
	if (!InItem.IsSupported())
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_AssetIsNotSupported", "Asset {0} is not supported and it can't be copied"), InItem.GetDisplayName()));
		return false;
	}

	// Cannot copy an item outside the paths known to this data source
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsUnknown", "Folder '{0}' is outside the mount roots of this data source"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be a content folder
	if (!IsKnownContentPath(InternalDestPath))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsNotContent", "Folder '{0}' is not a known content path"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be writable
	if (!ContentBrowserAssetData::CanModifyPath(AssetTools, InternalDestPath, OutErrorMsg))
	{
		return false;
	}

	// Destination must not be self (folder)
	const FName VirtualPath = InItem.GetVirtualPath();
	if (InDestPath == VirtualPath)
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("CopyError_ItemInsideItself", "Can't copy folder inside itself"));
		return false;
	}
	return true;
}

bool UContentBrowserAssetDataSource::CopyItem(const FContentBrowserItemData& InItem, const FName InDestPath)
{
	if (!InItem.IsSupported())
	{
		return false;
	}


	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	if (!IsKnownContentPath(InternalDestPath))
	{
		return false;
	}

	return ContentBrowserAssetData::CopyItems(AssetTools, this, MakeArrayView(&InItem, 1), InternalDestPath);
}

bool UContentBrowserAssetDataSource::BulkCopyItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	if (!IsKnownContentPath(InternalDestPath))
	{
		return false;
	}

	return ContentBrowserAssetData::CopyItems(AssetTools, this, InItems, InternalDestPath);
}

bool UContentBrowserAssetDataSource::CanMoveItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg)
{
	// Cannot move an item outside the paths known to this data source
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsUnknown", "Folder '{0}' is outside the mount roots of this data source"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be a content folder
	if (!IsKnownContentPath(InternalDestPath))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsNotContent", "Folder '{0}' is not a known content path"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be writable
	if (!ContentBrowserAssetData::CanModifyPath(AssetTools, InternalDestPath, OutErrorMsg))
	{
		return false;
	}

	// Moving has to be able to delete the original item
	if (!ContentBrowserAssetData::CanModifyItem(AssetTools, this, InItem, OutErrorMsg))
	{
		return false;
	}

	// Destination must not be self (folder)
	FName VirtualPath = InItem.GetVirtualPath();
	if (InDestPath == VirtualPath)
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("MoveError_ItemInsideItself", "Can't move Item inside itself"));
		return false;
	}

	// Cannot be moved to the same folder
	FString VirtualPathAsString = VirtualPath.ToString();
	int32 LastSlashIndex = INDEX_NONE;
	VirtualPathAsString.FindLastChar('/', LastSlashIndex);
	if (LastSlashIndex != INDEX_NONE)
	{
		VirtualPathAsString = VirtualPathAsString.Left(VirtualPathAsString.Len() - (VirtualPathAsString.Len() - LastSlashIndex));
	}

	if (InDestPath == FName(VirtualPathAsString))
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("MoveError_ItemInsideSamePath", "Can't move Item inside the same location"));
		return false;
	}

	return true;
}

bool UContentBrowserAssetDataSource::MoveItem(const FContentBrowserItemData& InItem, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	if (!IsKnownContentPath(InternalDestPath))
	{
		return false;
	}

	return ContentBrowserAssetData::MoveItems(AssetTools, this, MakeArrayView(&InItem, 1), InternalDestPath);
}

bool UContentBrowserAssetDataSource::BulkMoveItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	if (!IsKnownContentPath(InternalDestPath))
	{
		return false;
	}

	return ContentBrowserAssetData::MoveItems(AssetTools, this, InItems, InternalDestPath);
}

bool UContentBrowserAssetDataSource::AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return ContentBrowserAssetData::AppendItemReference(AssetRegistry, this, InItem, InOutStr);
}

bool UContentBrowserAssetDataSource::AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return ContentBrowserAssetData::AppendItemObjectPath(AssetRegistry, this, InItem, InOutStr);
}

bool UContentBrowserAssetDataSource::AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return ContentBrowserAssetData::AppendItemPackageName(AssetRegistry, this, InItem, InOutStr);
}

bool UContentBrowserAssetDataSource::UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	return ContentBrowserAssetData::UpdateItemThumbnail(this, InItem, InThumbnail);
}

bool UContentBrowserAssetDataSource::CanHandleDragDropEvent(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent) const
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InItem))
	{
		if (TSharedPtr<FExternalDragOperation> ExternalDragDropOp = InDragDropEvent.GetOperationAs<FExternalDragOperation>())
		{
			TOptional<EMouseCursor::Type> NewDragCursor;
			if (!ExternalDragDropOp->HasFiles() || !ContentBrowserAssetData::CanModifyPath(AssetTools, FolderPayload->GetInternalPath(), nullptr))
			{
				NewDragCursor = EMouseCursor::SlashedCircle;
			}
			else if (ExternalDragDropOp->HasFiles())
			{
				bool bSupportOneFile = false;
				for (const FString& File : ExternalDragDropOp->GetFiles())
				{
					FStringView Extension = FPathViews::GetExtension(File);
					if (Extension.IsEmpty() || AssetTools->IsImportExtensionAllowed(Extension))
					{
						bSupportOneFile = true;
					}
				}
				if (!bSupportOneFile)
				{
					NewDragCursor = EMouseCursor::SlashedCircle;
				}
			}
			ExternalDragDropOp->SetCursorOverride(NewDragCursor);

			return true; // We will handle this drop, even if the result is invalid (eg, read-only folder)
		}
	}

	return false;
}

bool UContentBrowserAssetDataSource::HandleDragEnterItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return CanHandleDragDropEvent(InItem, InDragDropEvent);
}

bool UContentBrowserAssetDataSource::HandleDragOverItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return CanHandleDragDropEvent(InItem, InDragDropEvent);
}

bool UContentBrowserAssetDataSource::HandleDragLeaveItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return CanHandleDragDropEvent(InItem, InDragDropEvent);
}

bool UContentBrowserAssetDataSource::HandleDragDropOnItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InItem))
	{
		if (TSharedPtr<FExternalDragOperation> ExternalDragDropOp = InDragDropEvent.GetOperationAs<FExternalDragOperation>())
		{
			FText ErrorMsg;
			FString UnsupportedFiles;
			if (ExternalDragDropOp->HasFiles() && ContentBrowserAssetData::CanModifyPath(AssetTools, FolderPayload->GetInternalPath(), &ErrorMsg))
			{
				TArray<FString> ImportFiles = ExternalDragDropOp->GetFiles();
			
				if (!ImportFiles.IsEmpty())
				{
					// Delay import until next tick to avoid blocking the process that files were dragged from
					GEditor->GetEditorSubsystem<UImportSubsystem>()->ImportNextTick(ImportFiles, FolderPayload->GetInternalPath().ToString());
				}
			}

			return true; // We handled this drop, even if the result was invalid (eg, read-only folder)
		}
	}

	return false;
}

bool UContentBrowserAssetDataSource::TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InItem))
	{
		OutCollectionId = AssetPayload->GetAssetData().GetSoftObjectPath();
		return true;
	}
	return false;
}

bool UContentBrowserAssetDataSource::Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath)
{
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InItem))
	{
		OutPackagePath = FolderPayload->GetInternalPath();
		return true;
	}
	return false;
}

bool UContentBrowserAssetDataSource::Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(InItem))
	{
		OutAssetData = AssetPayload->GetAssetData();
		return true;
	}
	return false;
}

bool UContentBrowserAssetDataSource::Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath)
{
	return IsKnownContentPath(InPackagePath) // Ignore unknown content paths
		&& TryConvertInternalPathToVirtual(InPackagePath, OutPath);
}

bool UContentBrowserAssetDataSource::Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath)
{
	return InAssetData.AssetClassPath != FTopLevelAssetPath(TEXT("/Script/CoreUObject"), TEXT("Class")) // Ignore legacy class items
		&& ContentBrowserAssetData::IsPrimaryAsset(InAssetData)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		&& TryConvertInternalPathToVirtual(InUseFolderPaths ? InAssetData.PackagePath : InAssetData.ObjectPath, OutPath);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UContentBrowserAssetDataSource::RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter)
{
	FilterCache.RemoveUnusedCachedData(IDOwner, InVirtualPathsInUse, DataFilter);
}

void UContentBrowserAssetDataSource::ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
	FilterCache.ClearCachedData(IDOwner);
}

bool UContentBrowserAssetDataSource::IsKnownContentPath(const FName InPackagePath) const
{
	FNameBuilder PackagePathStr(InPackagePath);
	const FStringView PackagePathStrView = PackagePathStr;

	const FCharacterNode* CurrentNode = &RootContentPathsTrie;

	for (const TCHAR& Character : PackagePathStrView)
	{
		const TPair<FCharacterNodePtr, int32>* NextNodePair = CurrentNode->NextNodes.Find(TChar<TCHAR>::ToLower(Character));

		if (!NextNodePair)
		{
			// This text start with no root content path
			return false;
		}
		 
		const FCharacterNode* NextNode = NextNodePair->Key.Get();

		// Is the next node terminal
		if (NextNode->bIsEndOfAMountPoint)
		{
			// The package path start with a root content path
			return true;
		}

		CurrentNode = NextNode;
	}
	
	/**
	 * Test if the folder is a root folder here like / Game.
	 * Where the only thing missing is the last '/'.
	 */
	return CurrentNode->NextNodes.Contains(TEXT('/'));
}

bool UContentBrowserAssetDataSource::GetObjectPathsForCollections(TArrayView<const FCollectionRef> InCollections, const bool bIncludeChildCollections, FCollectionEnumerationFunc* GetCollectionObjectPathsFunc, TArray<FSoftObjectPath>& OutObjectPaths)
{
	if (InCollections.Num() > 0)
	{
		const ECollectionRecursionFlags::Flags CollectionRecursionMode = bIncludeChildCollections ? ECollectionRecursionFlags::SelfAndChildren : ECollectionRecursionFlags::Self;
		
		for (const FCollectionRef& Collection : InCollections)
		{
			if (Collection.Container)
			{
				if (GetCollectionObjectPathsFunc != nullptr)
				{
					(*GetCollectionObjectPathsFunc)(Collection, CollectionRecursionMode, [&OutObjectPaths](const FSoftObjectPath& ObjectPath)
						{
							OutObjectPaths.Add(ObjectPath);
						});
				}
				else
				{
					Collection.Container->GetObjectsInCollection(Collection.Name, Collection.Type, OutObjectPaths, CollectionRecursionMode);
				}
			}
		}

		return true;
	}

	return false;
}

FContentBrowserItemData UContentBrowserAssetDataSource::CreateAssetFolderItem(const FName InInternalFolderPath)
{
	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(InInternalFolderPath, VirtualizedPath);

	const EContentBrowserFolderAttributes FolderAttributes = GetAssetFolderAttributes(InInternalFolderPath);
	const bool bIsCookedPath =
		EnumHasAnyFlags(
			FolderAttributes, EContentBrowserFolderAttributes::HasAssets | EContentBrowserFolderAttributes::HasRedirectors)
		&& !EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::HasSourceContent);
	const bool bIsPlugin = EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::IsInPlugin);
	return ContentBrowserAssetData::CreateAssetFolderItem(
		this, VirtualizedPath, InInternalFolderPath, bIsCookedPath, bIsPlugin);
}

FContentBrowserItemData UContentBrowserAssetDataSource::CreateAssetFileItem(const FAssetData& InAssetData)
{
	FName VirtualizedPath;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FName InternalPath = InAssetData.ObjectPath;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	TryConvertInternalPathToVirtual(InternalPath, VirtualizedPath);

	const EContentBrowserFolderAttributes FolderAttributes = GetAssetFolderAttributes(InAssetData.PackagePath);
	const bool bIsPlugin = EnumHasAnyFlags(FolderAttributes, EContentBrowserFolderAttributes::IsInPlugin);
	return ContentBrowserAssetData::CreateAssetFileItem(this, VirtualizedPath, InternalPath, InAssetData, bIsPlugin);
}

FContentBrowserItemData UContentBrowserAssetDataSource::CreateUnsupportedAssetFileItem(const FAssetData& InAssetData)
{
	FName VirtualizedPath;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FName InternalPath = InAssetData.ObjectPath;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	TryConvertInternalPathToVirtual(InternalPath, VirtualizedPath);

	return ContentBrowserAssetData::CreateUnsupportedAssetFileItem(this, VirtualizedPath, InternalPath, InAssetData);
}

TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> UContentBrowserAssetDataSource::GetAssetFolderItemPayload(
	const FContentBrowserItemData& InItem) const
{
	return ContentBrowserAssetData::GetAssetFolderItemPayload(this, InItem);
}

TSharedPtr<const FContentBrowserAssetFileItemDataPayload> UContentBrowserAssetDataSource::GetAssetFileItemPayload(const FContentBrowserItemData& InItem) const
{
	return ContentBrowserAssetData::GetAssetFileItemPayload(this, InItem);
}

TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> UContentBrowserAssetDataSource::GetUnsupportedAssetFileItemPayload(const FContentBrowserItemData& InItem) const
{
	return ContentBrowserAssetData::GetUnsupportedAssetFileItemPayload(this, InItem);
}

void UContentBrowserAssetDataSource::OnAssetRegistryFileLoadProgress(const IAssetRegistry::FFileLoadProgressUpdateData& InProgressUpdateData)
{
	if (InProgressUpdateData.bIsDiscoveringAssetFiles)
	{
		DiscoveryStatusText = FText::Format(LOCTEXT("DiscoveringAssetFiles", "Discovering Asset Files: {0} files found."), InProgressUpdateData.NumTotalAssets);
	}
	else
	{
		float ProgressFraction = 0.0f;
		if (InProgressUpdateData.NumTotalAssets > 0)
		{
			ProgressFraction = InProgressUpdateData.NumAssetsProcessedByAssetRegistry / (float)InProgressUpdateData.NumTotalAssets;
		}

		if (InProgressUpdateData.NumAssetsPendingDataLoad > 0)
		{
			DiscoveryStatusText = FText::Format(LOCTEXT("DiscoveringAssetData", "Discovering Asset Data ({0}): {1} assets remaining."), FText::AsPercent(ProgressFraction), InProgressUpdateData.NumAssetsPendingDataLoad);
		}
		else
		{
			const int32 NumAssetsLeftToProcess = InProgressUpdateData.NumTotalAssets - InProgressUpdateData.NumAssetsProcessedByAssetRegistry;
			if (NumAssetsLeftToProcess == 0)
			{
				DiscoveryStatusText = FText();
			}
			else
			{
				DiscoveryStatusText = FText::Format(LOCTEXT("ProcessingAssetData", "Processing Asset Data ({0}): {1} assets remaining."), FText::AsPercent(ProgressFraction), NumAssetsLeftToProcess);
			}
		}
	}
}

void UContentBrowserAssetDataSource::OnAssetsAdded(TConstArrayView<FAssetData> InAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnAssetAdded);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))

	FAssetPropertyTagCache& Cache = FAssetPropertyTagCache::Get();

	struct FUpdate
	{
		int32 Index;
		FContentBrowserItemDataUpdate ItemDataUpdate;
	};
	UE::TConsumeAllMpmcQueue<FUpdate> Updates;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnAssetAdded_TryCacheClass);

		// This is calling into some parts of the asset registry that are not thread-safe at the moment.
		for (const FAssetData& InAssetData : InAssets)
		{
			if (InAssetData.GetOptionalOuterPathName().IsNone())
			{
				Cache.TryCacheClass(InAssetData.AssetClassPath);
			}
		}
	}

	ParallelFor(InAssets.Num(),
		[this, &Cache, &InAssets, &Updates](int32 Index)
		{
			const FAssetData& InAssetData = InAssets[Index];
			UE_LOGF(LogContentBrowserAssetDataSource, VeryVerbose, "OnAssetsAdded: %ls", *WriteToString<256>(InAssetData.GetSoftObjectPath()));
			if (ContentBrowserAssetData::IsPrimaryAsset(InAssetData))
			{
				Updates.ProduceItem(Index, FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateAssetFileItem(InAssetData)));
			}
		}
	);

	Updates.ConsumeAllLifo(
		[this, &InAssets](FUpdate&& Update)
		{
			// The owner folder of this asset is no longer considered empty
			OnPathPopulated(InAssets[Update.Index]);

			QueueItemDataUpdate(MoveTemp(Update.ItemDataUpdate));
		}
	);
}

void UContentBrowserAssetDataSource::OnAssetRemoved(const FAssetData& InAssetData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnAssetRemoved);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	if (ContentBrowserAssetData::IsPrimaryAsset(InAssetData))
	{
		UE_LOGF(LogContentBrowserAssetDataSource, VeryVerbose, "OnAssetRemoved: %ls", *WriteToString<256>(InAssetData.GetSoftObjectPath()));
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(CreateAssetFileItem(InAssetData)));
	}
}

void UContentBrowserAssetDataSource::OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnAssetRenamed);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	if (ContentBrowserAssetData::IsPrimaryAsset(InAssetData))
	{
		UE_LOGF(LogContentBrowserAssetDataSource, VeryVerbose, "OnAssetRenamed: %ls", *WriteToString<256>(InAssetData.GetSoftObjectPath()));

		// The owner folder of this asset is no longer considered empty
		OnPathPopulated(InAssetData);

		FName VirtualizedPath;
		TryConvertInternalPathToVirtual(*InOldObjectPath, VirtualizedPath);

		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemMovedUpdate(CreateAssetFileItem(InAssetData), VirtualizedPath));
	}
}

void UContentBrowserAssetDataSource::OnAssetUpdated(const FAssetData& InAssetData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnAssetUpdated);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	if (ContentBrowserAssetData::IsPrimaryAsset(InAssetData))
	{
		UE_LOGF(LogContentBrowserAssetDataSource, VeryVerbose, "OnAssetUpdated: %ls", *WriteToString<256>(InAssetData.GetSoftObjectPath()));

		FAssetPropertyTagCache::Get().TryCacheClass(InAssetData.AssetClassPath);
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFileItem(InAssetData)));
	}
}

void UContentBrowserAssetDataSource::OnAssetUpdatedOnDisk(const FAssetData& InAssetData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnAssetUpdatedOnDisk);

	if (ContentBrowserAssetData::IsPrimaryAsset(InAssetData))
	{
		UE_LOGF(LogContentBrowserAssetDataSource, VeryVerbose, "OnAssetUpdatedOnDisk: %ls", *WriteToString<256>(InAssetData.GetSoftObjectPath()));

		FAssetPropertyTagCache::Get().TryCacheClass(InAssetData.AssetClassPath);
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFileItem(InAssetData)));
	}
}

void UContentBrowserAssetDataSource::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnObjectPropertyChanged);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	if (InObject && InObject->IsAsset() && ContentBrowserAssetData::IsPrimaryAsset(InObject))
	{
		FAssetData AssetData(InObject);
		FAssetPropertyTagCache::Get().TryCacheClass(AssetData.AssetClassPath);
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFileItem(AssetData)));
	}
}

void UContentBrowserAssetDataSource::OnObjectPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnObjectPreSave);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	if (InObject && InObject->IsAsset() && ContentBrowserAssetData::IsPrimaryAsset(InObject))
	{
		FAssetData AssetData(InObject);
		FAssetPropertyTagCache::Get().TryCacheClass(AssetData.AssetClassPath);
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFileItem(AssetData)));
	}
}

void UContentBrowserAssetDataSource::OnPathsAdded(TConstArrayView<FStringView> Paths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnPathsAdded);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	RecentlyPopulatedAssetFolders.Empty();
	for (FStringView InPath : Paths)
	{
		// Completely ignore paths that do not pass the most inclusive filter
		if (!ContentBrowserDataUtils::PathPassesAttributeFilter(InPath, 0, EContentBrowserItemAttributeFilter::IncludeAll))
		{
			continue;
		}

		FName PathName(InPath);
		const bool bIsPlugin = AssetViewUtils::IsPluginFolder(InPath);
		if (bIsPlugin)
		{
			OnPathPopulated(InPath, EContentBrowserFolderAttributes::IsInPlugin);
		}

		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateAssetFolderItem(PathName)));

		// Minus one because the test depth start at zero
		const int32 CurrentDepth = ContentBrowserDataUtils::CalculateFolderDepthOfPath(InPath) - 1;
		int32 Index;
		if (InPath.FindLastChar(TEXT('/'), Index))
		{ 
			uint32 PathNameHash = GetTypeHash(PathName);
			FName ParentPath(InPath.Left(Index));
			uint32 ParentPathHash = GetTypeHash(ParentPath);
			OnAssetPathAddedDelegate.Broadcast(PathName, InPath, PathNameHash, ParentPath, ParentPathHash, CurrentDepth);
		}
	}
	RecentlyPopulatedAssetFolders.Empty();
}

void UContentBrowserAssetDataSource::OnPathsRemoved(TConstArrayView<FStringView> Paths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAssetDataSource::OnPathsRemoved);
	LLM_SCOPE_BYNAME(TEXT("UContentBrowserAssetDataSource"))
	for (FStringView InPath : Paths)
	{
		// Deleted paths are no longer relevant for tracking
		FName PathName(InPath);
		RecentlyPopulatedAssetFolders.Remove(PathName);
		AssetFolderToAttributes.Remove(PathName);

		const FStringView MountPointWithLeadingSlash = FPathViews::GetMountPointNameFromPath(InPath, nullptr, false);
		if (FPackageName::MountPointExists(MountPointWithLeadingSlash))
		{
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(CreateAssetFolderItem(PathName)));
		}
		else
		{
			// Required to avoid loss of favorites for plugins being unmounted
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemUnmountedUpdate(CreateAssetFolderItem(PathName)));
		}

		OnAssetPathRemovedDelegate.Broadcast(PathName, GetTypeHash(PathName));
	}
}

void UContentBrowserAssetDataSource::OnPathPopulated(const FAssetData& InAssetData)
{
	EContentBrowserFolderAttributes FolderAttributes =
		InAssetData.IsRedirector() ? EContentBrowserFolderAttributes::HasRedirectors : EContentBrowserFolderAttributes::HasAssets;
	FolderAttributes |= (InAssetData.PackageFlags & PKG_Cooked ? EContentBrowserFolderAttributes::None : EContentBrowserFolderAttributes::HasSourceContent);

	if (InAssetData.GetAssetAccessSpecifier() == EAssetAccessSpecifier::EpicInternal)
	{
		if (IContentBrowserSingleton::Get().GetShowEpicInternalContentPermissionList()->PassesStartsWithFilter(FNameBuilder(InAssetData.PackageName)))
		{
			FolderAttributes |= EContentBrowserFolderAttributes::HasVisiblePublicContent;
		}
	}
	else if (InAssetData.GetAssetAccessSpecifier() == EAssetAccessSpecifier::Public)
	{
		FolderAttributes |= EContentBrowserFolderAttributes::HasVisiblePublicContent;
	}

	OnPathPopulated(FNameBuilder(InAssetData.PackagePath), FolderAttributes);
}

void UContentBrowserAssetDataSource::OnPathPopulated(const FStringView InPath, const EContentBrowserFolderAttributes InAttributesToSet)
{
	// Recursively un-hide this path, emitting update events for any paths that change state so that the view updates
	if (InPath.Len() > 1)
	{
		// Trim any trailing slash
		FStringView Path = InPath;
		if (Path[Path.Len() - 1] == TEXT('/'))
		{
			Path = Path.Left(Path.Len() - 1);
		}

		FName PathName(Path);

		// If we've already visited this path then we can assume we visited the parents as well
		// and can skip visiting this path and its parents
		if (const EContentBrowserFolderAttributes* RecentlyAddedFolderAttributesPtr = RecentlyPopulatedAssetFolders.Find(PathName);
			RecentlyAddedFolderAttributesPtr && EnumHasAllFlags(*RecentlyAddedFolderAttributesPtr, InAttributesToSet))
		{
			return;
		}

		// Recurse first as we want parents to be updated before their children
		if (int32 LastSlashIndex = INDEX_NONE;
			Path.FindLastChar(TEXT('/'), LastSlashIndex) && LastSlashIndex > 0)
		{
			OnPathPopulated(Path.Left(LastSlashIndex), InAttributesToSet);
		}

		// Unhide this folder and emit a notification if required
		if (SetAssetFolderAttributes(PathName, InAttributesToSet))
		{
			// Queue an update event for this path as it may have become visible in the view
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFolderItem(PathName)));
		}

		// Mark that this path has been visited
		RecentlyPopulatedAssetFolders.FindOrAdd(PathName) |= InAttributesToSet;
	}
}

void UContentBrowserAssetDataSource::OnAlwaysShowPath(const FString& InPath)
{
	// Recursively force show this path, emitting update events for any paths that change state so that the view updates
	if (InPath.Len() > 1)
	{
		// Trim any trailing slash
		FString Path = InPath;
		if (Path[Path.Len() - 1] == TEXT('/'))
		{
			Path.LeftInline(Path.Len() - 1);
		}

		// Recurse first as we want parents to be updated before their children
		if (int32 LastSlashIndex = INDEX_NONE;
			Path.FindLastChar(TEXT('/'), LastSlashIndex) && LastSlashIndex > 0)
		{
			OnAlwaysShowPath(Path.Left(LastSlashIndex));
		}

		// Force show this folder and emit a notification if required
		FName PathName(Path);
		if (SetAssetFolderAttributes(PathName, EContentBrowserFolderAttributes::AlwaysVisible))
		{
			// Queue an update event for this path as it may have become visible in the view
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFolderItem(PathName)));
		}
	}
}

void UContentBrowserAssetDataSource::BuildRootPathVirtualTree() 
{
	Super::BuildRootPathVirtualTree();

	for (const FString& RootContentPath : RootContentPaths)
	{
		RootPathAdded(RootContentPath);
	}
}

void UContentBrowserAssetDataSource::OnContentPathMounted(const FString& InAssetPath, const FString& InFileSystemPath)
{
	RootContentPaths.Add(InAssetPath);
	AddRootContentPathToStateMachine(InAssetPath);

	RootPathAdded(InAssetPath);

	// Mount roots are always visible
	OnAlwaysShowPath(InAssetPath);
}

void UContentBrowserAssetDataSource::OnContentPathDismounted(const FString& InAssetPath, const FString& InFileSystemPath)
{
	RootPathRemoved(InAssetPath);

	RemoveRootContentPathFromStateMachine(InAssetPath);
	RootContentPaths.Remove(InAssetPath);
}

EContentBrowserFolderAttributes UContentBrowserAssetDataSource::GetAssetFolderAttributes(const FName InPath) const
{
	const EContentBrowserFolderAttributes* FolderAttributesPtr = AssetFolderToAttributes.Find(InPath);
	return FolderAttributesPtr
		? *FolderAttributesPtr
		: EContentBrowserFolderAttributes::None;
}

bool UContentBrowserAssetDataSource::SetAssetFolderAttributes(const FName InPath, const EContentBrowserFolderAttributes InAttributesToSet)
{
	if (InAttributesToSet != EContentBrowserFolderAttributes::None)
	{
		EContentBrowserFolderAttributes& FolderAttributes = AssetFolderToAttributes.FindOrAdd(InPath);

		const EContentBrowserFolderAttributes PreviousAttributes = FolderAttributes;
		EnumAddFlags(FolderAttributes, InAttributesToSet);

		const bool bHasChanged = FolderAttributes != PreviousAttributes;
		if (bHasChanged)
		{
			const EContentBrowserFolderAttributes NewAttributes = InAttributesToSet & ~(PreviousAttributes);
			UE_LOGF(LogContentBrowserAssetDataSource, Verbose, "Updated folder attributes: %ls %ls", *WriteToString<256>(InPath), *WriteToString<256>(NewAttributes));
		}
		return bHasChanged;
	}

	return false;
}

bool UContentBrowserAssetDataSource::ClearAssetFolderAttributes(const FName InPath, const EContentBrowserFolderAttributes InAttributesToClear)
{
	if (InAttributesToClear != EContentBrowserFolderAttributes::None)
	{
		if (EContentBrowserFolderAttributes* FolderAttributesPtr = AssetFolderToAttributes.Find(InPath))
		{
			const EContentBrowserFolderAttributes PreviousAttributes = *FolderAttributesPtr;
			EnumRemoveFlags(*FolderAttributesPtr, InAttributesToClear);

			const bool bHasChanged = *FolderAttributesPtr != PreviousAttributes;
			if (*FolderAttributesPtr == EContentBrowserFolderAttributes::None)
			{
				AssetFolderToAttributes.Remove(InPath);
			}
			return bHasChanged;
		}
	}

	return false;
}

bool UContentBrowserAssetDataSource::HideFolderIfEmpty(const IContentBrowserHideFolderIfEmptyFilter& HideFolderIfEmptyFilter, FName Path, FStringView PathString) const
{
	if (!HideFolderIfEmptyFilter.HideFolderIfEmpty(Path, PathString))
	{
		return false;
	}

	// If any subpaths shouldn't be hidden, then Path should be visible.
	bool bAnySubPathVisible = false;
	AssetRegistry->EnumerateSubPaths(Path, [&HideFolderIfEmptyFilter, &bAnySubPathVisible](FName ChildPath)
		{
			FNameBuilder ChildPathBuilder(ChildPath);
			if (!HideFolderIfEmptyFilter.HideFolderIfEmpty(ChildPath, ChildPathBuilder))
			{
				bAnySubPathVisible = true;
				return false;
			}
			return true;
		}, true);
	return !bAnySubPathVisible;
}

void UContentBrowserAssetDataSource::PopulateAddNewContextMenu(UToolMenu* InMenu)
{
	const UContentBrowserDataMenuContext_AddNewMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_AddNewMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_AddNewMenu was missing!"));

	// Extract the internal asset paths that belong to this data source from the full list of selected paths given in the context
	TArray<FName> SelectedAssetPaths;
	for (const FName& SelectedPath : ContextObject->SelectedPaths)
	{
		FName InternalPath;
		if (TryConvertVirtualPathToInternal(SelectedPath, InternalPath) && IsKnownContentPath(InternalPath))
		{
			SelectedAssetPaths.Add(InternalPath);
		}
	}

	// Only add the asset items if we have an asset path selected
	FNewAssetContextMenu::FOnNewAssetRequested OnNewAssetRequested;
	FNewAssetContextMenu::FOnImportAssetRequested OnImportAssetRequested;
	if (SelectedAssetPaths.Num() > 0)
	{
		OnImportAssetRequested = FNewAssetContextMenu::FOnImportAssetRequested::CreateUObject(this, &UContentBrowserAssetDataSource::OnImportAsset);
		if (ContextObject->OnBeginItemCreation.IsBound())
		{
			OnNewAssetRequested = FNewAssetContextMenu::FOnNewAssetRequested::CreateUObject(this, &UContentBrowserAssetDataSource::OnNewAssetRequested, ContextObject->OnBeginItemCreation);
		}
	}

	FNewAssetContextMenu::MakeContextMenu(
		InMenu,
		SelectedAssetPaths,
		OnImportAssetRequested,
		OnNewAssetRequested
		);
}

void UContentBrowserAssetDataSource::PopulateContentBrowserToolBar(UToolMenu* InMenu)
{
	const UContentBrowserToolbarMenuContext* ContextObject = InMenu->FindContext<UContentBrowserToolbarMenuContext>();
	checkf(ContextObject, TEXT("Required context UContentBrowserToolbarMenuContext was missing!"));

	TSharedPtr<SWidget> ImportButton = nullptr;

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		SAssignNew(ImportButton, SActionButton)
			.ToolTipText(LOCTEXT("ImportTooltip", "Import assets from files to the currently selected folder"))
			.OnClicked_UObject(this, &UContentBrowserAssetDataSource::OnImportClicked, ContextObject)
			.IsEnabled_UObject(this, &UContentBrowserAssetDataSource::IsImportEnabled, ContextObject)
			.Icon(FAppStyle::Get().GetBrush("Icons.Import"))
			.Text(LOCTEXT("Import", "Import"));
	}
	else
	{
		SAssignNew(ImportButton, SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("ImportTooltip", "Import assets from files to the currently selected folder"))
			.ContentPadding(2)
			.OnClicked_UObject(this, &UContentBrowserAssetDataSource::OnImportClicked, ContextObject)
			.IsEnabled_UObject(this, &UContentBrowserAssetDataSource::IsImportEnabled, ContextObject)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Import"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(3, 0, 0, 0))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "NormalText")
					.Text(LOCTEXT("Import", "Import"))
				]
			];
	}

	FToolMenuSection& Section = InMenu->FindOrAddSection("New");

	Section.AddSeparator(NAME_None);

	FToolMenuEntry& ImportEntry = Section.AddEntry(
		FToolMenuEntry::InitWidget(
			"Import",
			ImportButton.ToSharedRef(),
			FText::GetEmpty(),
			true,
			false
		));

	ImportEntry.InsertPosition.Position = EToolMenuInsertType::Last;
}

void UContentBrowserAssetDataSource::PopulateAssetFolderContextMenu(UToolMenu* InMenu)
{
	return ContentBrowserAssetData::PopulateAssetFolderContextMenu(this, InMenu, *AssetFolderContextMenu);
}

void UContentBrowserAssetDataSource::PopulateAssetFileContextMenu(UToolMenu* InMenu)
{
	return ContentBrowserAssetData::PopulateAssetFileContextMenu(this, InMenu, *AssetFileContextMenu);
}

void UContentBrowserAssetDataSource::PopulateDragDropContextMenu(UToolMenu* InMenu)
{
	const UContentBrowserDataMenuContext_DragDropMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_DragDropMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_DragDropMenu was missing!"));

	FToolMenuSection& Section = InMenu->FindOrAddSection("MoveCopy");
	if (ContextObject->bCanCopy)
	{
		// Get the internal drop path
		FName DropAssetPath;
		{
			for (const FContentBrowserItemData& DropTargetItemData : ContextObject->DropTargetItem.GetInternalItems())
			{
				if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(DropTargetItemData))
				{
					DropAssetPath = FolderPayload->GetInternalPath();
					break;
				}
			}
		}

		// Extract the internal package paths that belong to this data source from the full list of selected items given in the context
		TArray<FName> AdvancedCopyInputs;
		for (const FContentBrowserItem& DraggedItem : ContextObject->DraggedItems)
		{
			for (const FContentBrowserItemData& DraggedItemData : DraggedItem.GetInternalItems())
			{
				if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = GetAssetFileItemPayload(DraggedItemData))
				{
					AdvancedCopyInputs.Add(AssetPayload->GetAssetData().PackageName);
				}

				if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(DraggedItemData))
				{
					AdvancedCopyInputs.Add(FolderPayload->GetInternalPath());
				}
			}
		}

		if (!DropAssetPath.IsNone() && AdvancedCopyInputs.Num() > 0)
		{
			Section.AddMenuEntry(
				"DragDropAdvancedCopy",
				LOCTEXT("DragDropAdvancedCopy", "Advanced Copy Here"),
				LOCTEXT("DragDropAdvancedCopyTooltip", "Copy the dragged items and any specified dependencies to this folder, afterwards fixing up any dependencies on copied files to the new files."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, AdvancedCopyInputs, DestinationPath = DropAssetPath.ToString()]() { OnAdvancedCopyRequested(AdvancedCopyInputs, DestinationPath); }))
				);
		}
	}
}

void UContentBrowserAssetDataSource::OnAdvancedCopyRequested(const TArray<FName>& InAdvancedCopyInputs, const FString& InDestinationPath)
{
	AssetTools->BeginAdvancedCopyPackages(InAdvancedCopyInputs, InDestinationPath / FString());
}

void UContentBrowserAssetDataSource::OnImportAsset(const FName InPath)
{
	if (ensure(!InPath.IsNone()))
	{
		AssetTools->ImportAssetsWithDialogAsync(InPath.ToString());
	}
}

void UContentBrowserAssetDataSource::OnNewAssetRequested(const FName InPath, TWeakObjectPtr<UClass> InFactoryClass, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation InOnBeginItemCreation)
{
	UClass* FactoryClass = InFactoryClass.Get();
	if (ensure(!InPath.IsNone()) && ensure(FactoryClass) && ensure(InOnBeginItemCreation.IsBound()))
	{
		UFactory* NewFactory = NewObject<UFactory>(GetTransientPackage(), FactoryClass);

		// This factory may get gc'd as a side effect of various delegates potentially calling CollectGarbage so protect against it from being gc'd out from under us
		FGCObjectScopeGuard FactoryGCGuard(NewFactory);
		
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, InPath.ToString());
		
		FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(NewFactory);
		if (NewFactory->ConfigureProperties())
		{
			FEditorDelegates::OnNewAssetCreated.Broadcast(NewFactory);

			const TOptional<FString> DefaultAssetName = AssetTools->GetDefaultAssetNameForClass(NewFactory->GetSupportedClass(), NewFactory);
			const FString BaseAssetName = DefaultAssetName.Get(NewFactory->GetDefaultNewAssetName());

			FString UniqueAssetName;
			FString PackageNameToUse;
			AssetTools->CreateUniqueAssetName(InPath.ToString() / BaseAssetName, FString(), PackageNameToUse, UniqueAssetName);

			OnBeginCreateAsset(*UniqueAssetName, InPath, NewFactory->GetSupportedClass(), NewFactory, InOnBeginItemCreation);
		}
	}
}

void UContentBrowserAssetDataSource::OnBeginCreateAsset(const FName InDefaultAssetName, const FName InPackagePath, UClass* InAssetClass, UFactory* InFactory, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation InOnBeginItemCreation)
{
	if (!ensure(InOnBeginItemCreation.IsBound()))
	{
		return;
	}

	if (!ensure(InAssetClass || InFactory))
	{
		return;
	}

	if (InAssetClass && InFactory && !ensure(InAssetClass->IsChildOf(InFactory->GetSupportedClass())))
	{
		return;
	}

	UClass* ClassToUse = InAssetClass ? InAssetClass : (InFactory ? InFactory->GetSupportedClass() : nullptr);
	if (!ensure(ClassToUse))
	{
		return;
	}

	FAssetPropertyTagCache::Get().TryCacheClass(FTopLevelAssetPath(ClassToUse));

	FContentBrowserItemPath AssetPathToUse = ContentBrowserModule->Get().GetInitialPathToSaveAsset(FContentBrowserItemPath(InPackagePath, EContentBrowserPathType::Internal));

	// This will create a content browser item, which will trigger the factory to create the actual object upon rename completion
	auto CreateContentBrowserItemWithCreationPayload = [this, InDefaultAssetName, InPackagePath, InAssetClass, InFactory, ClassToUse, InOnBeginItemCreation]()
	{
		FAssetData NewAssetData(*(InPackagePath.ToString() / InDefaultAssetName.ToString()), InPackagePath, InDefaultAssetName, ClassToUse->GetClassPathName());

		FName VirtualizedPath;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FName InternalPath = NewAssetData.ObjectPath;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		TryConvertInternalPathToVirtual(InternalPath, VirtualizedPath);

		FContentBrowserItemData NewItemData(
			this,
			EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Asset
				| EContentBrowserItemFlags::Temporary_Creation,
			VirtualizedPath,
			NewAssetData.AssetName,
			FText::AsCultureInvariant(NewAssetData.AssetName.ToString()),
			MakeShared<FContentBrowserAssetFileItemDataPayload_Creation>(MoveTemp(NewAssetData), InAssetClass, InFactory),
			{ InternalPath });

		InOnBeginItemCreation.Execute(FContentBrowserItemDataTemporaryContext(
			MoveTemp(NewItemData),
			FContentBrowserItemDataTemporaryContext::FOnValidateItem::CreateUObject(
				this, &UContentBrowserAssetDataSource::OnValidateItemName),
			FContentBrowserItemDataTemporaryContext::FOnFinalizeItem::CreateUObject(
				this, &UContentBrowserAssetDataSource::OnFinalizeCreateAsset)));
	};

	const bool bUseAsyncCreate = InFactory && EnumHasAnyFlags(InFactory->GetSupportedWorkflows(), EFactoryCreateWorkflow::Asynchronous);
	const bool bShowDialogToPickPath = !AssetPathToUse.HasInternalPath() || (AssetPathToUse.GetInternalPathName() != InPackagePath);
	if (bShowDialogToPickPath)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		const FString InitialInternalPath = AssetPathToUse.HasInternalPath() ? AssetPathToUse.GetInternalPathString() : TEXT("/Game");

		if (bUseAsyncCreate)
		{
			AssetToolsModule.Get().CreateAssetWithDialogAsync(InDefaultAssetName.ToString(), InitialInternalPath, ClassToUse, InFactory, FAssetCreateComplete(), FAssetCreateCancelled(), NAME_None);
		}
		else
		{
			AssetToolsModule.Get().CreateAssetWithDialog(InDefaultAssetName.ToString(), InitialInternalPath, ClassToUse, InFactory, NAME_None, /*bCallConfigureProperties*/ false);
		}
	}
	else if (bUseAsyncCreate)
	{
		FOnFactoryConfigurePropertiesAsyncComplete OnConfigurePropertiesComplete = FOnFactoryConfigurePropertiesAsyncComplete::CreateLambda([CreateContentBrowserItemWithCreationPayload](UFactory* Factory)
		{
			CreateContentBrowserItemWithCreationPayload();
		});

		// At this point, the configuration process is completely handed off to the factory. It's the factory's responsibility to call Complete or Cancel.
		// Complete will call the regular content browser creation code path.
		InFactory->ConfigurePropertiesAsync(OnConfigurePropertiesComplete, FOnFactoryConfigurePropertiesAsyncCancelled());
	}
	else
	{
		CreateContentBrowserItemWithCreationPayload();
	}
}

bool UContentBrowserAssetDataSource::OnValidateItemName(const FContentBrowserItemData& InItem, const FString& InProposedName, FText* OutErrorMsg)
{
	return CanRenameItem(InItem, &InProposedName, nullptr, OutErrorMsg);
}

FReply UContentBrowserAssetDataSource::OnImportClicked(const UContentBrowserToolbarMenuContext* ContextObject)
{
	// Extract the internal asset paths that belong to this data source from the full list of selected paths given in the context
	FName InternalPath;
	if (TryConvertVirtualPathToInternal(ContextObject->GetCurrentPath(), InternalPath) && IsKnownContentPath(InternalPath))
	{
		OnImportAsset(InternalPath);
	}

	return FReply::Handled();
}

bool UContentBrowserAssetDataSource::IsImportEnabled(const UContentBrowserToolbarMenuContext* ContextObject) const
{
	return ContextObject->CanWriteToCurrentPath();
}

FContentBrowserItemData UContentBrowserAssetDataSource::OnFinalizeCreateFolder(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg)
{
	checkf(InItemData.GetOwnerDataSource() == this, TEXT("OnFinalizeCreateFolder was bound to an instance from the wrong data source!"));
	checkf(EnumHasAllFlags(InItemData.GetItemFlags(), EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Temporary_Creation), TEXT("OnFinalizeCreateFolder called for an instance with the incorrect type flags!"));

	// Committed creation
	if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = GetAssetFolderItemPayload(InItemData))
	{
		const FString FolderPath = FPaths::GetPath(FolderPayload->GetInternalPath().ToString()) / InProposedName;

		FString NewPathOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename(FolderPath, NewPathOnDisk) && IFileManager::Get().MakeDirectory(*NewPathOnDisk, true))
		{
			AssetRegistry->AddPath(FolderPath);
			AssetViewUtils::OnAlwaysShowPath().Broadcast(FolderPath);
			return CreateAssetFolderItem(*FolderPath);
		}
	}

	ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FailedToCreateFolder", "Failed to create folder"));
	return FContentBrowserItemData();
}

FContentBrowserItemData UContentBrowserAssetDataSource::OnFinalizeCreateAsset(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg)
{
	checkf(InItemData.GetOwnerDataSource() == this, TEXT("OnFinalizeCreateAsset was bound to an instance from the wrong data source!"));
	checkf(EnumHasAllFlags(InItemData.GetItemFlags(), EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Temporary_Creation), TEXT("OnFinalizeCreateAsset called for an instance with the incorrect type flags!"));

	// Committed creation
	UObject* Asset = nullptr;
	{
		TSharedPtr<const FContentBrowserAssetFileItemDataPayload_Creation> CreationContext = StaticCastSharedPtr<const FContentBrowserAssetFileItemDataPayload_Creation>(InItemData.GetPayload());
		
		UClass* AssetClass = CreationContext->GetAssetClass();
		UFactory* Factory = CreationContext->GetFactory();
		
		if (AssetClass || Factory)
		{
			Asset = AssetTools->CreateAsset(InProposedName, CreationContext->GetAssetData().PackagePath.ToString(), AssetClass, Factory, FName("ContentBrowserNewAsset"));
		}
	}

	if (!Asset)
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FailedToCreateAsset", "Failed to create asset"));
		return FContentBrowserItemData();
	}

	return CreateAssetFileItem(FAssetData(Asset));
}

FContentBrowserItemData UContentBrowserAssetDataSource::OnFinalizeDuplicateAsset(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg)
{
	checkf(InItemData.GetOwnerDataSource() == this, TEXT("OnFinalizeDuplicateAsset was bound to an instance from the wrong data source!"));
	checkf(EnumHasAllFlags(InItemData.GetItemFlags(), EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Temporary_Duplication), TEXT("OnFinalizeDuplicateAsset called for an instance with the incorrect type flags!"));

	// Committed duplication
	UObject* Asset = nullptr;
	{
		TSharedPtr<const FContentBrowserAssetFileItemDataPayload_Duplication> DuplicationContext = StaticCastSharedPtr<const FContentBrowserAssetFileItemDataPayload_Duplication>(InItemData.GetPayload());

		if (UObject* SourceObject = DuplicationContext->GetSourceObject())
		{
			Asset = AssetTools->DuplicateAsset(InProposedName, DuplicationContext->GetAssetData().PackagePath.ToString(), SourceObject);
		}
	}

	if (!Asset)
	{
		ContentBrowserAssetData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FailedToCreateAsset", "Failed to create asset"));
		return FContentBrowserItemData();
	}

	return CreateAssetFileItem(FAssetData(Asset));
}

void UContentBrowserAssetDataSource::AddRootContentPathToStateMachine(const FString& InAssetPath)
{
	FCharacterNode* CurrentNode = &RootContentPathsTrie;

	for (const TCHAR& Character : InAssetPath)
	{
		TPair<FCharacterNodePtr,int32>& NextNode = CurrentNode->NextNodes.FindOrAdd(TChar<TCHAR>::ToLower(Character));
		++NextNode.Value;
		CurrentNode = NextNode.Key.Get();
	}

	CurrentNode->bIsEndOfAMountPoint = true;
}

void UContentBrowserAssetDataSource::RemoveRootContentPathFromStateMachine(const FString& InAssetPath)
{
	FCharacterNode* CurrentNode = &RootContentPathsTrie;

	for (const TCHAR& Character : InAssetPath)
	{
		const TCHAR LoweredCharacter = TChar<TCHAR>::ToLower(Character);
		uint32 Hash = GetTypeHash(LoweredCharacter);
		TPair<FCharacterNodePtr,int32>* NextNode = CurrentNode->NextNodes.FindByHash(Hash,LoweredCharacter);

		if (!NextNode)
		{
			return;
		}

		--NextNode->Value;
		if (NextNode->Value == 0)
		{
			CurrentNode->NextNodes.RemoveByHash(Hash,LoweredCharacter);
			return;
		}

		CurrentNode = NextNode->Key.Get();
	}

	CurrentNode->bIsEndOfAMountPoint = false;
}

void UContentBrowserAssetDataSource::OnExtendContentBrowserCommands(TSharedRef<FUICommandList> InCommandList, FOnContentBrowserGetSelection InGetSelectionDelegate)
{
	const FContentBrowserAssetDataSourceCommands& AssetDataSourceCommands = FContentBrowserAssetDataSourceCommands::Get();

	using namespace UE::Editor::ContentBrowserAssetDataSource;

	InCommandList->MapAction(
		AssetDataSourceCommands.CaptureThumbnail,
		FExecuteAction::CreateWeakLambda(this, [InGetSelectionDelegate]()
			{
				TArray<FAssetData> SelectedAssets;
				TArray<FString> SelectedFolders;
				InGetSelectionDelegate.ExecuteIfBound(SelectedAssets, SelectedFolders);
				const int32 Result = CaptureThumbnail(SelectedAssets);

				// Notify user
				FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
				FNotificationInfo Info(
					FText::FormatOrdered(
						LOCTEXT("CaptureThumbnailNotification", "Capturing thumbnail(s) for {0} asset(s)"), 
						FText::AsNumber(Result)
					)
				);
				Info.ExpireDuration = 4.f;
				NotificationManager.AddNotification(Info);
			}
		),
		FCanExecuteAction::CreateWeakLambda(this, [InGetSelectionDelegate]()->bool
			{
				TArray<FAssetData> SelectedAssets;
				TArray<FString> SelectedFolders;
				InGetSelectionDelegate.ExecuteIfBound(SelectedAssets, SelectedFolders);
				return CanCaptureThumbnail(SelectedAssets);
			}
		)
	);
}

bool UContentBrowserAssetDataSource::PathPassesCompiledDataFilter(const FContentBrowserCompiledAssetDataFilter& InFilter, const FName InInternalPath)
{
	if (InFilter.ExcludedPackagePaths.Contains(InInternalPath)) // PassesExcludedPathsFilter
	{
		return false;
	}

	FNameBuilder PathStr(InInternalPath);
	FStringView Path(PathStr);

	auto PathPassesFilter = [Path](const FPathPermissionList& InPathFilter, const bool InRecursive)
	{
		return !InPathFilter.HasFiltering() || (InRecursive ? InPathFilter.PassesStartsWithFilter(Path, /*bAllowParentPaths*/true) : InPathFilter.PassesFilter(Path));
	};

	return PathPassesFilter(InFilter.PackagePathsToInclude, InFilter.bRecursivePackagePathsToInclude)
		&& PathPassesFilter(InFilter.PackagePathsToExclude, InFilter.bRecursivePackagePathsToExclude)
		&& PathPassesFilter(InFilter.PathPermissionList, /*bRecursive*/ true)                         // PassesPathFilter
		&& ContentBrowserDataUtils::PathPassesAttributeFilter(Path, 0, InFilter.ItemAttributeFilter); // PassesAttributeFilter
}

UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::FAssetDataSourceFilterCache()
{
	UContentBrowserAssetDataSource::OnAssetPathAddedDelegate.AddRaw(this, &UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::OnPathAdded);
	UContentBrowserAssetDataSource::OnAssetPathRemovedDelegate.AddRaw(this, &UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::OnPathRemoved);
}

UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::~FAssetDataSourceFilterCache()
{
	UContentBrowserAssetDataSource::OnAssetPathAddedDelegate.RemoveAll(this);
	UContentBrowserAssetDataSource::OnAssetPathRemovedDelegate.RemoveAll(this);
}

bool UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::GetCachedCompiledInternalPaths(const FContentBrowserDataFilter& InFilter, FName InVirtualPath, TSet<FName>& OutCompiledInternalPath) const
{
	// We only use the cache if the query if is recursive
	if (InFilter.CacheID && InFilter.bRecursivePaths)
	{
		if (const FCachedDataPerID* CachedCompiledPathsForID = CachedCompiledInternalPaths.Find(InFilter.CacheID))
		{
			if (const TSet<FName>* CompiledPaths = CachedCompiledPathsForID->InternalPaths.Find(InVirtualPath))
			{
				OutCompiledInternalPath = *CompiledPaths;
				return true;
			}
		}
	}

	return false;
}

void UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::CacheCompiledInternalPaths(const FContentBrowserDataFilter& InFilter, FName InVirtualPath, const TSet<FName>& CompiledInternalPaths)
{
	// We only use the cache if the query if is recursive
	if (InFilter.CacheID && InFilter.bRecursivePaths)
	{
		FCachedDataPerID& CachedCompiledPathsForID = CachedCompiledInternalPaths.FindOrAdd(InFilter.CacheID);
		CachedCompiledPathsForID.InternalPaths.Add(InVirtualPath, CompiledInternalPaths);
		CachedCompiledPathsForID.ItemAttributeFilter = InFilter.ItemAttributeFilter;
	}
}

void UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::RemoveUnusedCachedData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter)
{
	// we always clear the cache for now. This should be improved in some future changes
	ClearCachedData(IDOwner);
}

void UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::ClearCachedData(const FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
	CachedCompiledInternalPaths.Remove(IDOwner);
}

void UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::Reset()
{
	CachedCompiledInternalPaths.Reset(); 
}

void UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::OnPathAdded(FName Path, FStringView PathString, uint32 PathHash, FName ParentPath, uint32 ParentPathHash, int32 PathDepth)
{
	for (TPair<FContentBrowserDataFilterCacheID, FCachedDataPerID>& CachedCompiledInternalPath : CachedCompiledInternalPaths)
	{
		if (ContentBrowserDataUtils::PathPassesAttributeFilter(PathString, PathDepth, CachedCompiledInternalPath.Value.ItemAttributeFilter))
		{
			for (TPair<FName, TSet<FName>>& CachedPaths : CachedCompiledInternalPath.Value.InternalPaths)
			{
				if (CachedPaths.Value.ContainsByHash(ParentPathHash, ParentPath))
				{
					CachedPaths.Value.AddByHash(PathHash, Path);
				}
			}
		}
	}
}

void UContentBrowserAssetDataSource::FAssetDataSourceFilterCache::OnPathRemoved(FName Path, uint32 PathHash)
{
	for (TPair<FContentBrowserDataFilterCacheID, FCachedDataPerID>& CachedCompiledInternalPath : CachedCompiledInternalPaths)
	{
		for (TPair<FName, TSet<FName>>& CachedPaths : CachedCompiledInternalPath.Value.InternalPaths)
		{
			CachedPaths.Value.RemoveByHash(PathHash, Path);
		}
	}
}

#undef LOCTEXT_NAMESPACE

