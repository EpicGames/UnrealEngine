// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAssetDataSourceCommands.h"

#define LOCTEXT_NAMESPACE "ContentBrowserAssetDataSourceCommands"

FContentBrowserAssetDataSourceCommands::FContentBrowserAssetDataSourceCommands()
	: TCommands<FContentBrowserAssetDataSourceCommands>(
		TEXT("ContentBrowserAssetDataSource"),
		LOCTEXT("ContentBrowserAssetDataSource", "Content Browser Asset Data Source"),
		NAME_None,
		FAppStyle::GetAppStyleSetName())
{
}

void FContentBrowserAssetDataSourceCommands::RegisterCommands()
{
	UI_COMMAND(
		CaptureThumbnail,
		"Capture Thumbnail",
		"Captures a custom thumbnail from the active viewport for the selected asset(s)",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::T, EModifierKey::Control | EModifierKey::Shift)
	);
}

#undef LOCTEXT_NAMESPACE
