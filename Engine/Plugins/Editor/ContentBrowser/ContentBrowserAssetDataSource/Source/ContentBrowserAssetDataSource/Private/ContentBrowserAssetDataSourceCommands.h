// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FContentBrowserAssetDataSourceCommands : public TCommands<FContentBrowserAssetDataSourceCommands>
{
public:
	FContentBrowserAssetDataSourceCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> CaptureThumbnail;
};
