// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "UObject/StrongObjectPtr.h"
#include "ContentBrowserAssetDataSource.h"
#include "ContentBrowserAssetDataSourceCommands.h"

class FContentBrowserAssetDataSourceModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
		if (GIsEditor && !IsRunningCommandlet())
		{
			AssetDataSource.Reset(NewObject<UContentBrowserAssetDataSource>(GetTransientPackage(), "AssetData"));
			AssetDataSource->Initialize();
			
			FContentBrowserAssetDataSourceCommands::Register();
		}
	}

	virtual void ShutdownModule() override
	{
		if (GIsEditor && !IsRunningCommandlet())
		{
			AssetDataSource.Reset();
			FContentBrowserAssetDataSourceCommands::Unregister();
		}
	}

private:
	TStrongObjectPtr<UContentBrowserAssetDataSource> AssetDataSource;
};

IMPLEMENT_MODULE(FContentBrowserAssetDataSourceModule, ContentBrowserAssetDataSource);
