// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAssetDataSourceUtils.h"

#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Components/Viewport.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"

int32 UE::Editor::ContentBrowserAssetDataSource::CaptureThumbnail(const TArray<FAssetData>& InAssets)
{
	if (InAssets.IsEmpty())
	{
		return 0;
	}

	if (!GCurrentLevelEditingViewportClient)
	{
		return 0;
	}

	FViewport* Viewport = GEditor->GetActiveViewport();
	if (!Viewport)
	{
		return 0;
	}

	{
		// Rerender the requested viewport
		FLevelEditorViewportClient* CachedViewportClient = GCurrentLevelEditingViewportClient;

		// Remove selection box around client during render
		GCurrentLevelEditingViewportClient = nullptr;
		Viewport->Draw();

		AssetViewUtils::CaptureThumbnailFromViewport(Viewport, InAssets);

		// Redraw viewport to have the yellow highlight again
		if (!GCurrentLevelEditingViewportClient)
		{
			GCurrentLevelEditingViewportClient = CachedViewportClient;
		}

		Viewport->Draw();
	}

	return InAssets.Num();
}

bool UE::Editor::ContentBrowserAssetDataSource::CanCaptureThumbnail(const TArray<FAssetData>& InAssets)
{
	if (GCurrentLevelEditingViewportClient == nullptr)
	{
		return false;
	}

	FAssetToolsModule* AssetToolsModule = FModuleManager::LoadModulePtr<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule)
	{
		return false;
	}

	return InAssets.ContainsByPredicate([AssetToolsModule](const FAssetData& InAsset)
		{
			return AssetToolsModule->Get().AssetUsesGenericThumbnail(InAsset);
		}
	);
}
