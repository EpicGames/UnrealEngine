// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Editor::ContentBrowser
{
	static bool IsNewStyleEnabled()
	{
		static bool bIsNewStyleEnabled = false;
		UE_CALL_ONCE([&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle")))
			{
				ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline, @see: UnrealEdGlobals.cpp, UE::Editor::ContentBrowser::EnableContentBrowserNewStyleCVarRegistration."));
				bIsNewStyleEnabled = CVar->GetBool();
			}
		});

		return bIsNewStyleEnabled;
	}
}
