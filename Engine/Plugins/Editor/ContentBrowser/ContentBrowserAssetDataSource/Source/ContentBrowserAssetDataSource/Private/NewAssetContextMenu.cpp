// Copyright Epic Games, Inc. All Rights Reserved.

#include "NewAssetContextMenu.h"

#include "ToolMenu.h"
#include "Widgets/SBoxPanel.h"
#include "ToolMenuEntry.h"
#include "Widgets/SOverlay.h"
#include "ToolMenuSection.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Factories/Factory.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "ContentBrowserUtils.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Styling/StyleColors.h"
#include "Widgets/SAssetMenuIcon.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

struct FFactoryItem
{
	UFactory& Factory;
	FText DisplayName;

	FFactoryItem(UFactory& InFactory, const FText& InDisplayName)
		: Factory(InFactory)
		, DisplayName(InDisplayName)
	{
	}
};

struct FWizardItem
{
	FText DisplayName;
	FText Description;
	FSlateIcon Icon;
	FSimpleDelegate OnClicked;
};

struct FCategoryChildren
{
	FCategoryChildren(FName InSubCategoryName, TSharedPtr<FCategorySubMenuItem> InSubMenuItem)
		: SubCategoryName(InSubCategoryName)
		, SubMenuItem(InSubMenuItem)
	{}

	FName SubCategoryName;
	TSharedPtr<FCategorySubMenuItem> SubMenuItem;
};

struct FFactoryInfo
{
	explicit FFactoryInfo(const FFactoryItem& InFactory)
		: Factory(InFactory)
		, SubSectionName(FText::GetEmpty())
	{}

	explicit FFactoryInfo(const FFactoryItem& InFactory, const FText& InSubSectionName)
		: Factory(InFactory)
		, SubSectionName(InSubSectionName)
	{}

	FFactoryItem Factory;
	FText SubSectionName;
};

struct FCategorySubMenuItem
{
	FText Name;
	TArray<FFactoryInfo> Factories;
	TArray<FWizardItem> Wizards;
	ECategoryMenuType CategoryMenuType = ECategoryMenuType::Menu;
	TArray<FCategoryChildren> Children;

	void GenerateChildrenSubMenuItemArray(TArray<TSharedPtr<FCategorySubMenuItem>>& OutSubMenuItems)
	{
		for (const FCategoryChildren& Child : Children)
		{
			if (Child.SubMenuItem.IsValid())
			{
				OutSubMenuItems.Add(Child.SubMenuItem);
			}
		}
	}

	void SortSubMenus(const FText& InMainCategory,FCategorySubMenuItem* SubMenu = nullptr)
	{
		if (!SubMenu)
		{
			SubMenu = this;
		}

		// Sort the factories by display name
		SubMenu->Factories.Sort([InMainCategory](const FFactoryInfo& A, const FFactoryInfo& B) -> bool
		{
			if (A.SubSectionName.CompareTo(B.SubSectionName) == 0)
			{
				return A.Factory.DisplayName.CompareToCaseIgnored(B.Factory.DisplayName) < 0;
			}

			if (B.SubSectionName.CompareToCaseIgnored(InMainCategory) == 0)
			{
				return false;
			}

			if (A.SubSectionName.CompareToCaseIgnored(InMainCategory) == 0)
			{
				return true;
			}

			return A.SubSectionName.CompareTo(B.SubSectionName) < 0;
		});

		for (FCategoryChildren& Child : SubMenu->Children)
		{
			if (Child.SubMenuItem.IsValid())
			{
				FCategorySubMenuItem* MenuData = Child.SubMenuItem.Get();
				SortSubMenus(InMainCategory, MenuData);
			}
		}
	}
};

/** Utility to return the new asset factories from FAssetToolsModule */
static const TArray<UFactory*> GetNewAssetFactories()
{
	QUICK_SCOPE_CYCLE_COUNTER(GetNewAssetFactories);

	static const FName NAME_AssetTools = "AssetTools";
	const IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(NAME_AssetTools).Get();

	return AssetTools.GetNewAssetFactories();
}

/**
 * Utility to find the factories (from the set provided by the caller) with a given category.
 * 
 * @param Factories			The factories to look in
 * @param AssetTypeCategory	The category to find factories for
 * @param FindFirstOnly		Returns once the first factory has been found
 */
static TArray<FFactoryItem> FindFactoriesInCategory(const TArray<UFactory*>& Factories, EAssetTypeCategories::Type AssetTypeCategory, bool FindFirstOnly)
{
	QUICK_SCOPE_CYCLE_COUNTER(FindFactoriesInCategory);
	
	TArray<FFactoryItem> FactoriesInThisCategory;

	for (UFactory* Factory : Factories)
	{
		QUICK_SCOPE_CYCLE_COUNTER(GetMenuCategories);
	
		const uint32 FactoryCategories = Factory->GetMenuCategories();
		if (FactoryCategories & AssetTypeCategory)
		{
			FactoriesInThisCategory.Emplace(*Factory, Factory->GetDisplayName());

			if (FindFirstOnly)
			{
				return FactoriesInThisCategory;
			}
		}
	}

	return FactoriesInThisCategory;
}

/** 
 * Utility to find the new assert factories with a given category.
 * 
 * @param AssetTypeCategory	The category to find factories for
 * @param FindFirstOnly		Returns once the first factory has been found
 */
static TArray<FFactoryItem> FindFactoriesInCategory(EAssetTypeCategories::Type AssetTypeCategory, bool FindFirstOnly)
{	
	return FindFactoriesInCategory(GetNewAssetFactories(), AssetTypeCategory, FindFirstOnly);
}

class SFactoryMenuEntry : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SFactoryMenuEntry)
		: _IconContainerSize(32, 32)
		, _IconSize(28, 28)
	{}
		SLATE_ARGUMENT(FVector2D, IconContainerSize)
		SLATE_ARGUMENT(FVector2D, IconSize)
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	Factory				The factory this menu entry represents
	 */
	void Construct(const FArguments& InArgs, const UFactory* Factory)
	{
		const TSharedPtr<SWidget> IconContainer =
			SNew(SAssetMenuIcon,
				Factory->GetSupportedClass(),
				UE::Editor::ContentBrowser::IsNewStyleEnabled() ? Factory->GetNewAssetIconOverride() : Factory->GetNewAssetThumbnailOverride())
			.IconContainerSize(InArgs._IconContainerSize)
			.IconSize(InArgs._IconSize);

		static const FMargin IconSlotPadding = UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? FMargin(2, 0, 3, 0)	// Consistent with SMenuEntryBlock::BuildMenuEntryWidget, but accounts for icon size that is larger than the default
			: FMargin(0, 0, 0, 1);

		static const FMargin LabelSlotPadding = UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? FMargin(4, 0, 6, 0)	// Consistent with SMenuEntryBlock::BuildMenuEntryWidget
			: FMargin(4, 0, 4, 0);

		// The vertical padding between each menu entry
		static constexpr float VerticalEntryPadding = 4.0f;

		// Represents the default icon size in a menu entry, not the one used in this widget
		static constexpr float DefaultMenuIconSize = 14.0f;

		// Adjust the vertical padding to match the default menu entry spacing as much as possible, while keeping a desired padding minimum of 1px
		static const float VerticalPaddingAdjustment = ((InArgs._IconContainerSize.Y - DefaultMenuIconSize) / 2.0f) - VerticalEntryPadding;

		static const FMargin ChildSlotPadding = UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? FMargin(0, -VerticalPaddingAdjustment, 0, -VerticalPaddingAdjustment) // Offset to align with regular menu entries
			: FMargin(0);

		ChildSlot
		.Padding(ChildSlotPadding)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(IconSlotPadding)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				IconContainer.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(LabelSlotPadding)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(UE::Editor::ContentBrowser::IsNewStyleEnabled() ? FMargin(0) : FMargin(0, 0, 0, 1))
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("LevelViewportContextMenu.AssetLabel.Text.Font"))
					.Text(Factory->GetDisplayName())
				]
			]
		];

		SetToolTip(IDocumentation::Get()->CreateToolTip(Factory->GetToolTip(), nullptr, Factory->GetToolTipDocumentationPage(), Factory->GetToolTipDocumentationExcerpt()));
	}
};

void FNewAssetContextMenu::MakeContextMenu(
	UToolMenu* Menu,
	const TArray<FName>& InSelectedAssetPaths,
	const FOnImportAssetRequested& InOnImportAssetRequested,
	const FOnNewAssetRequested& InOnNewAssetRequested
	)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ContentBrowser_MakeNewAssetContextMenu);
	
	if (InSelectedAssetPaths.Num() == 0)
	{
		return;
	}

	static const FName NAME_AssetTools = "AssetTools";
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(NAME_AssetTools);

	// Ensure we can modify assets at these paths
	{
		TArray<FString> SelectedAssetPathStrs;
		for (const FName& SelectedPath : InSelectedAssetPaths)
		{
			SelectedAssetPathStrs.Add(SelectedPath.ToString());
		}

		if (!AssetToolsModule.Get().AllPassWritableFolderFilter(SelectedAssetPathStrs))
		{
			return;
		}
	}

	const FCanExecuteAction CanExecuteAssetActionsDelegate = FCanExecuteAction::CreateLambda([NumSelectedAssetPaths = InSelectedAssetPaths.Num()]()
	{
		// We can execute asset actions when we only have a single asset path selected
		return NumSelectedAssetPaths == 1;
	});

	const FName FirstSelectedPath = (InSelectedAssetPaths.Num() > 0) ? InSelectedAssetPaths[0] : FName();

	// Import
	if (InOnImportAssetRequested.IsBound() && !FirstSelectedPath.IsNone())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ContentBrowser_ImportSection);
	
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("ContentBrowserGetContent");
			Section.AddMenuEntry(
				"ImportAsset",
				LOCTEXT("ImportAsset", "Import to Current Folder"),
				LOCTEXT("ImportAssetTooltip_NewAsset", "Imports an asset from file to this folder."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import"),
				FUIAction(
					FExecuteAction::CreateStatic(&FNewAssetContextMenu::ExecuteImportAsset, InOnImportAssetRequested, FirstSelectedPath),
					CanExecuteAssetActionsDelegate
					)
				).InsertPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);
		}
	}


	if (InOnNewAssetRequested.IsBound())
	{
		// Add Create Section
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			FToolMenuSection& Section = Menu->AddSection("ContentBrowserNewAsset", LOCTEXT("CreateAssetsMenuHeading", "Create"));
		}

		// Add Basic Asset
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ContentBrowser_BasicSection);

			const FText CreateBasicAssetSectionLabel =
				UE::Editor::ContentBrowser::IsNewStyleEnabled()
				? FText::GetEmpty()
				: LOCTEXT("CreateBasicAssetsMenuHeading", "Create Basic Asset");

			FToolMenuSection& Section = Menu->AddSection("ContentBrowserNewBasicAsset", CreateBasicAssetSectionLabel);

			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				// When a section label is empty, it has no visual representation, so manually insert a separator
				Section.AddSeparator(NAME_None);
			}

			CreateNewAssetMenuCategory(
				Menu,
				"ContentBrowserNewBasicAsset",
				EAssetTypeCategories::Basic,
				FirstSelectedPath,
				InOnNewAssetRequested,
				CanExecuteAssetActionsDelegate
				);


		}

		// Add Advanced Asset
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ContentBrowser_AdvancedSection);

			const FText CreateAdvancedAssetSectionLabel =
				UE::Editor::ContentBrowser::IsNewStyleEnabled()
				? FText::GetEmpty()
				: LOCTEXT("CreateAdvancedAssetsMenuHeading", "Create Advanced Asset");

			FToolMenuSection& Section = Menu->AddSection("ContentBrowserNewAdvancedAsset", CreateAdvancedAssetSectionLabel);

			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				// When a section label is empty, it has no visual representation, so manually insert a separator
				Section.AddSeparator(NAME_None);
			}

			TArray<FAdvancedAssetCategory> AdvancedAssetCategories;
			AssetToolsModule.Get().GetAllAdvancedAssetCategories(AdvancedAssetCategories);
			AdvancedAssetCategories.Sort([](const FAdvancedAssetCategory& A, const FAdvancedAssetCategory& B) {
				return (A.CategoryName.CompareToCaseIgnored(B.CategoryName) < 0);
			});

			const IAssetTools& AssetTools = AssetToolsModule.Get();
			const TArray<UFactory*> NewAssetFactories = AssetTools.GetNewAssetFactories();
			
			
			for (const FAdvancedAssetCategory& AdvancedAssetCategory : AdvancedAssetCategories)
			{
				const bool FindFirstOnly = true;
				TArray<FFactoryItem> Factories = FindFactoriesInCategory(NewAssetFactories, AdvancedAssetCategory.CategoryType, FindFirstOnly);
				if (Factories.Num() > 0)
				{
					FToolMenuEntry& SubMenuEntry =
							Section.AddSubMenu(
							NAME_None,
							AdvancedAssetCategory.CategoryName,
							FText::GetEmpty(),
							FNewToolMenuDelegate::CreateStatic(
								&FNewAssetContextMenu::CreateNewAssetMenuCategory,
								FName("Section"),
								AdvancedAssetCategory.CategoryType,
								FirstSelectedPath,
								InOnNewAssetRequested,
								FCanExecuteAction() // We handle this at this level, rather than at the sub-menu item level
							),
							FUIAction(
								FExecuteAction(),
								CanExecuteAssetActionsDelegate
							),
							EUserInterfaceActionType::Button
						);

					SubMenuEntry.SubMenuData.Style.StyleName = "ContentBrowser.AddNewMenu";
				}
			}
		}
	}
}

void FNewAssetContextMenu::CreateNewAssetMenuCategory(UToolMenu* Menu, FName SectionName, EAssetTypeCategories::Type AssetTypeCategory, FName InPath, FOnNewAssetRequested InOnNewAssetRequested, FCanExecuteAction InCanExecuteAction)
{
	// Find UFactory classes that can create new objects in this category.
	const bool FindFirstOnly = false;
	TArray<FFactoryItem> FactoriesInThisCategory = FindFactoriesInCategory(AssetTypeCategory, FindFirstOnly);
	if (FactoriesInThisCategory.Num() == 0)
	{
		return;
	}

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	TArray<FAdvancedAssetCategory> CategoryNames;
	AssetToolsModule.Get().GetAllAdvancedAssetCategories(CategoryNames);
	FAdvancedAssetCategory* Category = CategoryNames.FindByPredicate([AssetTypeCategory](const FAdvancedAssetCategory& InAdvancedCategory)
		{
			return InAdvancedCategory.CategoryType == AssetTypeCategory;
		});
	// Misc will be used as the default category if the category can't be found
	FName CategoryName = Category ? FName(Category->CategoryName.BuildSourceString()) : FName(TEXT("Misc"));

	TSharedPtr<FCategorySubMenuItem> ParentMenuData = MakeShared<FCategorySubMenuItem>();
	for (FFactoryItem& Item : FactoriesInThisCategory)
	{
		FCategorySubMenuItem* SubMenu = ParentMenuData.Get();
		// If we need to populate the basic section we can skip everything else as there is not a category for it to iterate.
		if (AssetTypeCategory == EAssetTypeCategories::Basic)
		{
			SubMenu->Factories.Add(FFactoryInfo(Item));
			continue;
		}

		TArray<FAssetCategoryPath> CategoryPaths = Item.Factory.GetAssetMenuPathsForCategory(CategoryName);

		// Always add a default category path based on the major one so that SubCategory is set
		if (CategoryPaths.IsEmpty())
		{
			// Try to add the sub categories from the sub menus array
			const TArray<FText> SubCategories = Item.Factory.GetMenuCategorySubMenus();
			if (!SubCategories.IsEmpty())
			{
				for (const FText& SubCategory : SubCategories)
				{
					CategoryPaths.Add(FAssetCategoryPath(FText::FromName(CategoryName), SubCategory));
				}
			}
			else
			{
				CategoryPaths.Add(FAssetCategoryPath(FText::FromName(CategoryName)));
			}
		}

		// Final Section name to use in the menu for the entry
		TArray<FText> SubCategoriesSection;

		for (const FAssetCategoryPath& CategoryPath : CategoryPaths)
		{
			TArray<FCategoryPath> OutSubCategories;
			CategoryPath.GetSubCategoriesInfo(OutSubCategories);
			// For new full category path reset the category section and SubMenu
			SubCategoriesSection.Empty();
			SubMenu = ParentMenuData.Get();

			// All basic section will have the basic category
			if (OutSubCategories.IsEmpty())
			{
				OutSubCategories.Add(FCategoryPath(LOCTEXT("BasicSectionName", "Basic"), ECategoryMenuType::Section));
			}

			for (FCategoryPath SubCategory : OutSubCategories)
			{
				// If this is a sub-sub... menu remove the old section entries, this is currently only possible with the old AssetTypeAction.
				if (SubMenu != ParentMenuData.Get() && SubMenu->CategoryMenuType == ECategoryMenuType::Menu)
				{
					SubCategoriesSection.Empty();
				}

				const FString SourceString = SubCategory.GetSubMenuName().BuildSourceString();
				const FName SourceName = FName(SourceString);
				const FText SubCategorySectionName = SubCategory.GetSubMenuName();
				if (!SubCategorySectionName.IsEmpty())
				{
					SubCategoriesSection.Add(SubCategorySectionName);
				}

				FCategoryChildren* SubMenuData = SubMenu->Children.FindByPredicate(
					[SourceName, CategoryType = SubCategory.GetCategoryMenuType()] 
						(const FCategoryChildren& InCategoryChild)
					{
						// Get the entry that match the name and type
						return InCategoryChild.SubCategoryName == SourceName && 
								InCategoryChild.SubMenuItem.IsValid() && 
								InCategoryChild.SubMenuItem->CategoryMenuType == CategoryType;
					});

				if (SubMenuData)
				{
					SubMenu = SubMenuData->SubMenuItem.Get();
				}
				else
				{
					TSharedPtr<FCategorySubMenuItem> NewSubMenu = MakeShared<FCategorySubMenuItem>();
					NewSubMenu->Name = SubCategory.GetSubMenuName();
					NewSubMenu->CategoryMenuType = SubCategory.GetCategoryMenuType();
					SubMenu->Children.Add(FCategoryChildren(SourceName, NewSubMenu));
					SubMenu = NewSubMenu.Get();
				}
			}
			
			// Assign the factory after a category path is read so that if the asset is inside 2 different section/menu in the same category it will be assigned correctly
			if (SubCategoriesSection.IsEmpty())
			{
				SubMenu->Factories.Add(FFactoryInfo(Item, FText::FromName(CategoryName)));
			}
			else
			{
				for (const FText& SubCategorySection : SubCategoriesSection)
				{
					SubMenu->Factories.Add(FFactoryInfo(Item, SubCategorySection));
				}
			}
		}
	}
	ParentMenuData->SortSubMenus(FText::FromName(CategoryName));

	// Find wizards
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	for (const FContentBrowserModule::FWizard& Wizard : ContentBrowserModule.GetWizards())
	{
		if (Wizard.CategoryPath.CategoryType == AssetTypeCategory)
		{
			FCategorySubMenuItem* SubMenu = ParentMenuData.Get();

			const FString SourceString = Wizard.CategoryPath.CategoryName.BuildSourceString();
			const FName SourceName = FName(SourceString);
			FCategoryChildren* SubMenuData = SubMenu->Children.FindByPredicate([SourceName] (const FCategoryChildren& InCategoryChild)
				{
					// Get the entry that match the name and type
					return InCategoryChild.SubCategoryName == SourceName &&
							InCategoryChild.SubMenuItem.IsValid() && 
							// Wizard do not currently support the menu type or sub section name so consider them all in the menu category which was the default before
							InCategoryChild.SubMenuItem->CategoryMenuType == ECategoryMenuType::Menu;
				});

			if (SubMenuData && SubMenuData->SubMenuItem.IsValid())
			{
				SubMenu = SubMenuData->SubMenuItem.Get();
			}
			else
			{
				TSharedPtr<FCategorySubMenuItem> NewSubMenu = MakeShared<FCategorySubMenuItem>();
				NewSubMenu->Name = Wizard.CategoryPath.CategoryName;
				SubMenu->Children.Add(FCategoryChildren(SourceName, NewSubMenu));
				SubMenu = NewSubMenu.Get();
			}

			FWizardItem& WizardItem = SubMenu->Wizards.AddDefaulted_GetRef();
			WizardItem.DisplayName = Wizard.DisplayName;
			WizardItem.Description = Wizard.Description;
			WizardItem.Icon = Wizard.Icon;
			WizardItem.OnClicked = Wizard.OnOpen;
		}
	}

	CreateNewAssetMenus(Menu, SectionName, ParentMenuData, InPath, InOnNewAssetRequested, InCanExecuteAction);
}

void FNewAssetContextMenu::CreateNewAssetMenus(UToolMenu* Menu, FName SectionName, TSharedPtr<FCategorySubMenuItem> SubMenuData, FName InPath, FOnNewAssetRequested InOnNewAssetRequested, FCanExecuteAction InCanExecuteAction)
{
	QUICK_SCOPE_CYCLE_COUNTER(CreateNewAssetMenus);

	FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);

	for (const FWizardItem& WizardItem : SubMenuData->Wizards)
	{
		Section.AddMenuEntry(
			NAME_None,
			WizardItem.DisplayName,
			WizardItem.Description,
			WizardItem.Icon,
			FUIAction(FExecuteAction::CreateLambda([WizardItem]()
			{
				WizardItem.OnClicked.ExecuteIfBound();
			})));
	}

	if (!SubMenuData->Wizards.IsEmpty())
	{
		Section.AddSeparator(NAME_None);
	}

	for (const FFactoryInfo& FactoryInfo : SubMenuData->Factories)
	{
		FToolMenuSection* SectionToUse = &Section;

		if (!FactoryInfo.SubSectionName.IsEmpty())
		{
			const FName EntrySectionName = FName(FactoryInfo.SubSectionName.BuildSourceString());
			SectionToUse = &Menu->FindOrAddSection(EntrySectionName, FactoryInfo.SubSectionName);
			// Add the label anyway since it may have been created without it before
			SectionToUse->Label = FactoryInfo.SubSectionName;
		}

		const FFactoryItem& FactoryItem = FactoryInfo.Factory;

		TWeakObjectPtr<UClass> WeakFactoryClass = FactoryItem.Factory.GetClass();

		FName AssetTypeName;

		if (UClass* SupportedClass = FactoryItem.Factory.GetSupportedClass())
		{
			AssetTypeName = SupportedClass->GetFName();
		}

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			static constexpr uint32 IconContainerSize = 24;
			static constexpr uint32 IconSize = 16;

			FToolMenuEntry& Entry = SectionToUse->AddEntry(
				FToolMenuEntry::InitMenuEntry(
				NAME_None,
				FUIAction(
					FExecuteAction::CreateStatic(&FNewAssetContextMenu::ExecuteNewAsset, InOnNewAssetRequested, InPath, WeakFactoryClass),
					InCanExecuteAction
				),
			SNew(SFactoryMenuEntry, &FactoryItem.Factory)
				.IconContainerSize(FVector2D(IconContainerSize, IconContainerSize))
				.IconSize(FVector2D(IconSize, IconSize))
				.AddMetaData<FTagMetaData>(FTagMetaData(AssetTypeName)))
			);

			Entry.SubMenuData.Style.StyleName = "ContentBrowser.AddNewMenu";
		}
		else
		{
			SectionToUse->AddEntry(
				FToolMenuEntry::InitMenuEntry(
				NAME_None,
				FUIAction(
					FExecuteAction::CreateStatic(&FNewAssetContextMenu::ExecuteNewAsset, InOnNewAssetRequested, InPath, WeakFactoryClass),
					InCanExecuteAction
				),
			SNew(SFactoryMenuEntry, &FactoryItem.Factory)
				.AddMetaData<FTagMetaData>(FTagMetaData(AssetTypeName)))
			);
		}
	}

	if (SubMenuData->Children.Num() == 0)
	{
		return;
	}

	TArray<TSharedPtr<FCategorySubMenuItem>> SortedMenus;
	SubMenuData->GenerateChildrenSubMenuItemArray(SortedMenus);
	FText BasicSection = LOCTEXT("BasicSection", "Basic");
	SortedMenus.Sort([BasicSection](const TSharedPtr<FCategorySubMenuItem>& A, const TSharedPtr<FCategorySubMenuItem>& B) -> bool
	{
		// Basic section should always be on top
		if (B->Name.CompareToCaseIgnored(BasicSection) == 0)
		{
			return false;
		}

		// Basic section should always be on top
		if (A->Name.CompareToCaseIgnored(BasicSection) == 0)
		{
			return true;
		}

		// SubMenus should go after sections
		if (A->CategoryMenuType != B->CategoryMenuType)
		{
			return A->CategoryMenuType == ECategoryMenuType::Section;
		}

		return A->Name.CompareToCaseIgnored(B->Name) < 0;
	});

	bool bWasSeparatorAdded = false;
	for (TSharedPtr<FCategorySubMenuItem>& ChildMenuData : SortedMenus)
	{
		check(ChildMenuData.IsValid());

		if (ChildMenuData->CategoryMenuType == ECategoryMenuType::Section)
		{
			const FName SeparatorName = FName(ChildMenuData->Name.BuildSourceString());
			CreateNewAssetMenus(
				Menu,
				SeparatorName,
				ChildMenuData,
				InPath,
				InOnNewAssetRequested,
				InCanExecuteAction
				);
		}
		else
		{
			FToolMenuSection& NewSubMenuSection = Menu->FindOrAddSection(FName(TEXT("SubMenuSection")), FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));
			FName SubMenuName = FName(ChildMenuData->Name.BuildSourceString());
			if (!bWasSeparatorAdded)
			{
				bWasSeparatorAdded = true;
				NewSubMenuSection.AddSeparator(NAME_None);
			}

			FToolMenuEntry& Entry = NewSubMenuSection.AddSubMenu(
				SubMenuName,
				ChildMenuData->Name,
				FText::GetEmpty(),
				FNewToolMenuDelegate::CreateStatic(
					&FNewAssetContextMenu::CreateNewAssetMenus,
					SubMenuName,
					ChildMenuData,
					InPath,
					InOnNewAssetRequested,
					InCanExecuteAction
				),
				FUIAction(
					FExecuteAction(),
					InCanExecuteAction
				),
				EUserInterfaceActionType::Button
			);

			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				Entry.SubMenuData.Style.StyleName = "ContentBrowser.AddNewMenu";
			}
		}
	}
}

void FNewAssetContextMenu::ExecuteImportAsset(FOnImportAssetRequested InOnInportAssetRequested, FName InPath)
{
	InOnInportAssetRequested.ExecuteIfBound(InPath);
}

void FNewAssetContextMenu::ExecuteNewAsset(FOnNewAssetRequested InOnNewAssetRequested, FName InPath, TWeakObjectPtr<UClass> FactoryClass)
{
	if (ensure(FactoryClass.IsValid()) && ensure(!InPath.IsNone()))
	{
		InOnNewAssetRequested.ExecuteIfBound(InPath, FactoryClass);
	}
}

#undef LOCTEXT_NAMESPACE
