// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

class FReply;

class UToolMenu;
class SWindow;
class SWidget;

class FAssetFileContextMenu : public TSharedFromThis<FAssetFileContextMenu>
{
public:
	DECLARE_DELEGATE_OneParam(FOnShowAssetsInPathsView, const TArray<FAssetData>& /*AssetsToFind*/);
	DECLARE_DELEGATE(FOnRefreshView);

	/** Makes the context menu widget */
	UE_API void MakeContextMenu(
		UToolMenu* InMenu,
		const TArray<FAssetData>& InSelectedAssets,
		const FOnShowAssetsInPathsView& InOnShowAssetsInPathsView
		);

private:
	struct FSourceAssetsState
	{
		TSet<FSoftObjectPath> SelectedAssets;
		TSet<FSoftObjectPath> CurrentAssets;
	};

	struct FLocalizedAssetsState
	{
		FCulturePtr Culture;
		TSet<FSoftObjectPath> CurrentAssets;
	};

private:
	/** Helper to load selected assets and sort them by UClass */
	UE_API void GetSelectedAssetsByClass(TMap<UClass*, TArray<FAssetData> >& OutSelectedAssetsByClass) const;

	/** Helper to collect resolved filepaths for all selected assets */
	UE_API void GetSelectedAssetSourceFilePaths(TArray<FString>& OutFilePaths, TArray<FString>& OutUniqueSourceFileLabels, int32 &OutValidSelectedAssetCount) const;

	/** Handler to check to see if a imported asset actions should be visible in the menu */
	UE_API bool AreImportedAssetActionsVisible() const;

	/** Handler to check to see if imported asset actions are allowed */
	UE_API bool CanExecuteImportedAssetActions(const TArray<FString> ResolvedFilePaths) const;

	/** Handler to check to see if reimport asset actions are allowed */
	UE_API bool CanExecuteReimportAssetActions(const TArray<FString> ResolvedFilePaths) const;

	/** Handler for Reimport */
	UE_API void ExecuteReimport(int32 SourceFileIndex = INDEX_NONE);

	UE_API void ExecuteReimportWithNewFile(int32 SourceFileIndex = INDEX_NONE);

	/** Handler for FindInExplorer */
	UE_API void ExecuteFindSourceInExplorer(const TArray<FString> ResolvedFilePaths);

	/** Handler for OpenInExternalEditor */
	UE_API void ExecuteOpenInExternalEditor(const TArray<FString> ResolvedFilePaths);

	/** Handler to check to see if a load command is allowed */
	UE_API bool CanExecuteLoad() const;

	/** Handler for Load */
	UE_API void ExecuteLoad();

	/** Handler to check to see if a reload command is allowed */
	UE_API bool CanExecuteReload() const;

	/** Handler for Reload */
	UE_API void ExecuteReload();

	/** Handler to check to see if a reload uncooked command is allowed */
	bool CanExecuteReloadUncooked() const;

	/** Handler for Reload Uncooked */
	void ExecuteReloadUncooked();

	/** Is allowed to modify files or folders under this path */
	UE_API bool CanModifyPath(const FString& InPath) const;

	/** Adds options to menu */
	UE_API void AddMenuOptions(UToolMenu* Menu);

	/** Adds asset type-specific menu options to a menu builder. Returns true if any options were added. */
	UE_API bool AddImportedAssetMenuOptions(UToolMenu* Menu);
	
	/** Adds common menu options to a menu builder. Returns true if any options were added. */
	UE_API bool AddCommonMenuOptions(UToolMenu* Menu);

	/** Adds Asset Actions sub-menu to a menu builder. */
	UE_API void MakeAssetActionsSubMenu(UToolMenu* Menu);

	/** Handler to check to see if "Asset Actions" are allowed */
	UE_API bool CanExecuteAssetActions() const;

	/** Adds Asset Localization sub-menu to a menu builder. */
	UE_API void MakeAssetLocalizationSubMenu(UToolMenu* Menu);

	/** Adds the Create Localized Asset sub-menu to a menu builder. */
	UE_API void MakeCreateLocalizedAssetSubMenu(UToolMenu* Menu, TSet<FSoftObjectPath> InSelectedSourceAssets, TArray<FLocalizedAssetsState> InLocalizedAssetsState);

	/** Adds the Show Localized Assets sub-menu to a menu builder. */
	UE_API void MakeShowLocalizedAssetSubMenu(UToolMenu* Menu, TArray<FLocalizedAssetsState> InLocalizedAssetsState);

	/** Adds the Edit Localized Assets sub-menu to a menu builder. */
	UE_API void MakeEditLocalizedAssetSubMenu(UToolMenu* Menu, TArray<FLocalizedAssetsState> InLocalizedAssetsState);

	/** Create new localized assets for the given culture */
	UE_API void ExecuteCreateLocalizedAsset(TSet<FSoftObjectPath> InSelectedSourceAssets, FLocalizedAssetsState InLocalizedAssetsStateForCulture);

	/** Find the given assets in the Content Browser */
	UE_API void ExecuteFindInAssetTree(TArray<FSoftObjectPath> InAssets);

	/** Open the given assets in their respective editors */
	UE_API void ExecuteOpenEditorsForAssets(TArray<FSoftObjectPath> InAssets);

	/** Adds asset documentation menu options to a menu builder. Returns true if any options were added. */
	UE_API bool AddDocumentationMenuOptions(UToolMenu* Menu);
	
	/** Creates a sub-menu of Chunk IDs that are are assigned to all selected assets */
	UE_API void MakeChunkIDListMenu(UToolMenu* Menu);

	/** Handler for when create using asset is selected */
	UE_API void ExecuteCreateBlueprintUsing();

	/** Handler for when "Find in World" is selected */
	UE_API void ExecuteFindAssetInWorld();

	/** Handler for when "Property Matrix..." is selected */
	UE_API void ExecutePropertyMatrix();

	/** Handler for when "Show MetaData" is selected */
	UE_API void ExecuteShowAssetMetaData();

	/** Handler for when "Copy Asset Registry Tags" is selected */
	void ExecuteCopyAssetRegistryTags();

	/** Handler for when "Diff Selected" is selected */
	UE_API void ExecuteDiffSelected() const;

	/** Handler for Consolidate */
	UE_API void ExecuteConsolidate();

	/** Handler for Capture Thumbnail */
	UE_DEPRECATED(5.8, "Use UE::Editor::ContentBrowserAssetDataSource instead")
	UE_API void ExecuteCaptureThumbnail();

	/** Handler for Clear Thumbnail */
	UE_API void ExecuteClearThumbnail();

	/** Handler for when "Migrate Asset" is selected */
	UE_API void ExecuteMigrateAsset();

	/** Handler for GoToAssetCode */
	UE_API void ExecuteGoToCodeForAsset(UClass* SelectedClass);

	/** Handler for GoToAssetDocs */
	UE_API void ExecuteGoToDocsForAsset(UClass* SelectedClass);

	/** Handler for GoToAssetDocs */
	UE_API void ExecuteGoToDocsForAsset(UClass* SelectedClass, const FString ExcerptSection);

#if USE_STABLE_LOCALIZATION_KEYS
	/** Handler for resetting the localization ID of the current selection */
	UE_API void ExecuteResetLocalizationId();
#endif

	/** Handler for showing the cached list of localized texts stored in the package header */
	UE_API void ExecuteShowLocalizationCache(const FAssetData InAsset, const FString InPackageFilename);

	/** Handler for Export */
	UE_API void ExecuteExport();

	/** Handler for Bulk Export */
	UE_API void ExecuteBulkExport();

	/** Handler to assign ChunkID to a selection of assets */
	UE_API void ExecuteAssignChunkID();

	/** Handler to remove all ChunkID assignments from a selection of assets */
	UE_API void ExecuteRemoveAllChunkID();

	/** Handler to remove a single ChunkID assignment from a selection of assets */
	UE_API void ExecuteRemoveChunkID(int32 ChunkID);

	/** Handler to export the selected asset(s) to experimental text format */
	UE_API void ExportSelectedAssetsToText();

	/** Handler to export the selected asset(s) to experimental text format */
	UE_API void ViewSelectedAssetAsText();
	UE_API bool CanViewSelectedAssetAsText() const;

	/** Run the rountrip test on this asset */
	UE_API void DoTextFormatRoundtrip();

	/** Handler to check if we can create blueprint using selected asset */
	UE_API bool CanExecuteCreateBlueprintUsing() const;

	/** Handler to check to see if a find in world command is allowed */
	UE_API bool CanExecuteFindAssetInWorld() const;

	/** Handler to check to see if a properties command is allowed */
	UE_API bool CanExecuteProperties() const;

	/** Handler to check to see if a property matrix command is allowed */
	UE_API bool CanExecutePropertyMatrix(FText& OutErrorMessage) const;
	UE_API bool CanExecutePropertyMatrix() const;

	/** Handler to check to see if "Capture Thumbnail" can be executed */
	UE_DEPRECATED(5.8, "Use UE::Editor::ContentBrowserAssetDataSource instead")
	UE_API bool CanExecuteCaptureThumbnail() const;

	/** Handler to check to see if "Clear Thumbnail" should be visible */
	UE_API bool CanClearCustomThumbnails() const;

	/** Initializes some variable used to in "CanExecute" checks that won't change at runtime or are too expensive to check every frame. */
	UE_API void CacheCanExecuteVars();

	/** Helper function to gather the package names of all selected assets */
	UE_API void GetSelectedPackageNames(TArray<FString>& OutPackageNames) const;

	/** Helper function to gather the packages containing all selected assets */
	UE_API void GetSelectedPackages(TArray<UPackage*>& OutPackages) const;

	/** Update internal state logic */
	UE_API void OnChunkIDAssignChanged(int32 ChunkID);

	/** Gets the current value of the ChunkID entry box */
	UE_API TOptional<int32> GetChunkIDSelection() const;

	/** Handles when the Assign chunkID dialog OK button is clicked */
	UE_API FReply OnChunkIDAssignCommit(TSharedPtr<SWindow> Window);

	/** Handles when the Assign chunkID dialog Cancel button is clicked */
	UE_API FReply OnChunkIDAssignCancel(TSharedPtr<SWindow> Window);

	/** Generates tooltip for the Property Matrix menu option */
	UE_API FText GetExecutePropertyMatrixTooltip() const;

	/** Generates a list of selected assets in the content browser */
	UE_API void GetSelectedAssets(TArray<UObject*>& Assets, bool SkipRedirectors) const;

	/** Generates a list of selected assets in the content browser, and returns the asset data so you do not have to load them */
	UE_API void GetSelectedAssetData(TArray<FAssetData>& AssetDataList, bool SkipRedirectors) const;

private:
	TArray<FAssetData> SelectedAssets;

	TWeakPtr<SWidget> ParentWidget;

	FOnShowAssetsInPathsView OnShowAssetsInPathsView;
	FOnRefreshView OnRefreshView;

	/** Cached CanExecute vars */
	bool bAtLeastOneNonRedirectorSelected = false;

	/** */
	int32 ChunkIDSelected = 0;
};

#undef UE_API
