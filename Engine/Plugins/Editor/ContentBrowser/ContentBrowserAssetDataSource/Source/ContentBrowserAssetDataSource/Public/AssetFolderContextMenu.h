// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

class UToolMenu;

class FAssetFolderContextMenu : public TSharedFromThis<FAssetFolderContextMenu>
{
public:
	UE_DEPRECATED(5.1, "Use the new overload.")
	UE_API void MakeContextMenu(
		UToolMenu* InMenu,
		const TArray<FString>& InSelectedPackagePaths
	);

	/** 
	 * Makes the context menu widget
	 * @param InMenu The menu in which the entries will be added
	 * @param InSelectedPackagePaths The path of the selected folders
	 * @param InSelectedPackages The package names of the selected asset item. Empty if the selection is only some folder items.
	 */
	UE_API void MakeContextMenu(
		UToolMenu* InMenu,
		const TArray<FString>& InSelectedPackagePaths,
		const TArray<FString>& InSelectedPackages
		);

private:
	/** Makes the asset tree context menu widget */
	UE_API void AddMenuOptions(UToolMenu* Menu);

	/** Handler for when "Migrate Folder" is selected */
	UE_API void ExecuteMigrateFolder();

	/** Handler for when "Fix up Redirectors in Folder" is selected */
	UE_API void ExecuteFixUpRedirectorsInFolder();

	/** Gets the first selected path, if it exists */
	UE_API FString GetFirstSelectedPath() const;

private:
	TArray<FString> SelectedPaths;

	// Not null if the selection contained some assets also
	TArray<FString> SelectedPackages;
};

#undef UE_API
