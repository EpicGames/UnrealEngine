// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

/**
 * Singleton cache of property tags data for each asset class.
 */
class FAssetPropertyTagCache
{
public:
	struct FPropertyTagCache
	{
		/** The kind of data represented by this tag value */
		UObject::FAssetRegistryTag::ETagType TagType = UObject::FAssetRegistryTag::TT_Hidden;

		/** Flags giving hints at how to display this tag value in the UI (see ETagDisplay) */
		uint32 DisplayFlags = UObject::FAssetRegistryTag::TD_None;

		/** Resolved display name of the associated tag */
		FText DisplayName;

		/** Optional tooltip of the associated tag */
		FText TooltipText;

		/** Optional suffix to apply to values of the tag attribute in the UI */
		FText Suffix;

		/** Optional value which denotes which values should be considered "important" in the UI */
		FString ImportantValue;
	};

	struct FClassPropertyTagCache
	{
		friend class FAssetPropertyTagCache;

	public:
		const FPropertyTagCache* GetCacheForTag(const FName InTagName) const
		{
			return TagNameToCachedDataMap.Find(InTagName);
		}

		/** See whether the given name is a known alias of a tag, and if so, return the real tag name */
		FName GetTagNameFromAlias(const FName InTagName) const
		{
			return DisplayNameToTagNameMap.FindRef(InTagName);
		}

	private:
		/** Map of an internal tag name to its cached data */
		TSortedMap<FName, FPropertyTagCache, FDefaultAllocator, FNameFastLess> TagNameToCachedDataMap;

		/** Map of a tag display name back to its real internal name */
		TSortedMap<FName, FName, FDefaultAllocator, FNameFastLess> DisplayNameToTagNameMap;
	};

	/** Get the singleton instance */
	static UE_API FAssetPropertyTagCache& Get();

	/** Try and populate the cache for the given class if it is loaded */
	UE_API void TryCacheClass(FTopLevelAssetPath InClassName);

	/** Try and populate the cache with classes which were not reigstered when TryCacheClass was called. */
	UE_API void CachePendingClasses();

	/** Get (or populate) the cache for the given asset class */
	UE_API const FClassPropertyTagCache& GetCacheForClass(FTopLevelAssetPath InClassName);

	/** Get the cache for the given asset class if it has been created */
	UE_API const FClassPropertyTagCache* FindCacheForClass(FTopLevelAssetPath InClassName);

private:
	mutable FRWLock Lock;

	/** 
	 * Mapping of the asset class name to its cache. 
	 * Values are not modified after construction, so they can safely be returned after releasing the lock. 
	 */
	TMap<FTopLevelAssetPath, TSharedPtr<FClassPropertyTagCache>> ClassToCacheMap;

	/** Classes we'd like to register but were not loaded when an asset of that type was last scanned */
	TSet<FTopLevelAssetPath> PendingClasses;
};

#undef UE_API
