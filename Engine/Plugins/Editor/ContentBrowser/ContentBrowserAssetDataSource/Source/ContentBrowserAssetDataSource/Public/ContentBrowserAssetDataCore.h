// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"
#include "UObject/Package.h"

class FContentBrowserAssetFileItemDataPayload;
class FContentBrowserAssetFolderItemDataPayload;
class FContentBrowserUnsupportedAssetFileItemDataPayload;

class IAssetTools;
class IAssetRegistry;
class FAssetThumbnail;
class FAssetFolderContextMenu;
class FAssetFileContextMenu;
class FContentBrowserModule;
class UObject;
class UToolMenu;
class UContentBrowserDataSource;

DECLARE_LOG_CATEGORY_EXTERN(LogContentBrowserAssetDataSource, Log, Log);

namespace ContentBrowserAssetData
{

CONTENTBROWSERASSETDATASOURCE_API FContentBrowserItemData CreateAssetFolderItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalFolderPath,
	const bool bIsCookedPath = false,
	const bool bIsPlugin = false);

CONTENTBROWSERASSETDATASOURCE_API FContentBrowserItemData CreateAssetFileItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalPath,
	const FAssetData& InAssetData,
	const bool bIsPlugin = false);

CONTENTBROWSERASSETDATASOURCE_API FContentBrowserItemData CreateUnsupportedAssetFileItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalPath,
	const FAssetData& InAssetData);

	CONTENTBROWSERASSETDATASOURCE_API TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> GetAssetFolderItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERASSETDATASOURCE_API TSharedPtr<const FContentBrowserAssetFileItemDataPayload> GetAssetFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERASSETDATASOURCE_API TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> GetUnsupportedAssetFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERASSETDATASOURCE_API void EnumerateAssetFolderItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>&)> InFolderPayloadCallback);

	CONTENTBROWSERASSETDATASOURCE_API void EnumerateAssetFileItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFileItemDataPayload>&)> InAssetPayloadCallback);

	CONTENTBROWSERASSETDATASOURCE_API void EnumerateAssetItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFileItemDataPayload>&)> InAssetPayloadCallback);

	CONTENTBROWSERASSETDATASOURCE_API void EnumerateAssetItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserAssetFileItemDataPayload>&)> InAssetPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>&)> InUnsupportedAssetPayloadCallback);

	CONTENTBROWSERASSETDATASOURCE_API bool IsPrimaryAsset(const FAssetData& InAssetData);

	CONTENTBROWSERASSETDATASOURCE_API bool IsPrimaryAsset(UObject* InObject);

	CONTENTBROWSERASSETDATASOURCE_API void SetOptionalErrorMessage(FText* OutErrorMsg, FText InErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanModifyPath(IAssetTools* InAssetTools, const FName InFolderPath, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanModifyItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanModifyAssetFolderItem(IAssetTools* InAssetTools, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanModifyAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanEditItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanEditAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool EditItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERASSETDATASOURCE_API bool EditAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads);

	CONTENTBROWSERASSETDATASOURCE_API bool CanViewItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanViewAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool ViewItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERASSETDATASOURCE_API bool ViewAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads);

	CONTENTBROWSERASSETDATASOURCE_API bool CanPreviewItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanPreviewAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool PreviewItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERASSETDATASOURCE_API bool PreviewAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads);

	CONTENTBROWSERASSETDATASOURCE_API bool CanDuplicateItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanDuplicateAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool DuplicateItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, UObject*& OutSourceAsset, FAssetData& OutNewAsset);

	CONTENTBROWSERASSETDATASOURCE_API bool DuplicateAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, UObject*& OutSourceAsset, FAssetData& OutNewAsset);

	CONTENTBROWSERASSETDATASOURCE_API bool DuplicateItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TArray<FAssetData>& OutNewAssets);

	CONTENTBROWSERASSETDATASOURCE_API bool DuplicateAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, TArray<FAssetData>& OutNewAssets);

	CONTENTBROWSERASSETDATASOURCE_API bool CanSaveItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanSaveAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool SaveItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags);

	CONTENTBROWSERASSETDATASOURCE_API bool SaveAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, const EContentBrowserItemSaveFlags InSaveFlags);

	CONTENTBROWSERASSETDATASOURCE_API bool CanDeleteItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanDeleteAssetFolderItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanDeleteAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanDeleteUnsupportedAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool DeleteItems(FContentBrowserModule* InContentBrowserModule, IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERASSETDATASOURCE_API bool DeleteAssetFolderItems(FContentBrowserModule* InContentBrowserModule, TArrayView<const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>> InFolderPayloads);

	CONTENTBROWSERASSETDATASOURCE_API bool DeleteAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads);

	CONTENTBROWSERASSETDATASOURCE_API bool DeleteUnsupportedAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserUnsupportedAssetFileItemDataPayload>> InUnsupportedAssetPayloads);

	CONTENTBROWSERASSETDATASOURCE_API bool CanPrivatizeItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanPrivatizeAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool PrivatizeItems(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private);

	CONTENTBROWSERASSETDATASOURCE_API bool CanRenameItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString* InNewName, FText* OutErrorMsg);
	CONTENTBROWSERASSETDATASOURCE_API bool CanRenameItem(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanRenameAssetFolderItem(IAssetTools* InAssetTools, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const FString* InNewName, FText* OutErrorMsg);
	CONTENTBROWSERASSETDATASOURCE_API bool CanRenameAssetFolderItem(IAssetTools* InAssetTools, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool CanRenameAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const FString* InNewName, const bool InIsTempoarary, FText* OutErrorMsg);
	CONTENTBROWSERASSETDATASOURCE_API bool CanRenameAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const bool bCheckUniqueName, const FString* InNewName, const bool InIsTempoarary, FText* OutErrorMsg);

	CONTENTBROWSERASSETDATASOURCE_API bool RenameItem(IAssetTools* InAssetTools, IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString& InNewName);

	CONTENTBROWSERASSETDATASOURCE_API bool RenameAssetFolderItem(IAssetRegistry* InAssetRegistry, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const FString& InNewName);

	CONTENTBROWSERASSETDATASOURCE_API bool RenameAssetFileItem(IAssetTools* InAssetTools, const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const FString& InNewName);

	CONTENTBROWSERASSETDATASOURCE_API bool CopyItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath);

	CONTENTBROWSERASSETDATASOURCE_API bool CopyAssetFolderItems(TArrayView<const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>> InFolderPayloads, const FName InDestPath);

	CONTENTBROWSERASSETDATASOURCE_API bool CopyAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, const FName InDestPath);

	CONTENTBROWSERASSETDATASOURCE_API bool MoveItems(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath);

	CONTENTBROWSERASSETDATASOURCE_API bool MoveAssetFolderItems(TArrayView<const TSharedRef<const FContentBrowserAssetFolderItemDataPayload>> InFolderPayloads, const FName InDestPath);

	CONTENTBROWSERASSETDATASOURCE_API bool MoveAssetFileItems(TArrayView<const TSharedRef<const FContentBrowserAssetFileItemDataPayload>> InAssetPayloads, const FName InDestPath);

	CONTENTBROWSERASSETDATASOURCE_API bool IsItemDirty(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERASSETDATASOURCE_API bool IsAssetFileItemDirty(const FContentBrowserAssetFileItemDataPayload& InAssetPayload);

	CONTENTBROWSERASSETDATASOURCE_API bool UpdateItemThumbnail(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail);

	CONTENTBROWSERASSETDATASOURCE_API bool UpdateAssetFileItemThumbnail(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FAssetThumbnail& InThumbnail);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendItemReference(IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendItemObjectPath(IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendItemPackageName(IAssetRegistry* InAssetRegistry, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendAssetFolderItemReference(IAssetRegistry* InAssetRegistry, const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendAssetFileItemReference(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendAssetFileObjectPath(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendAssetFilePackageName(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendUnsupportedAssetFileItemReference(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendUnsupportedAssetFileObjectPath(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool AppendUnsupportedAssetFilePackageName(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& InOutStr);

	CONTENTBROWSERASSETDATASOURCE_API bool GetItemPhysicalPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& OutDiskPath);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetFolderItemPhysicalPath(const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, FString& OutDiskPath);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetFileItemPhysicalPath(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, FString& OutDiskPath);

	CONTENTBROWSERASSETDATASOURCE_API bool GetUnsupportedAssetFileItemPhysicalPath(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, FString& OutDiskPath);

	CONTENTBROWSERASSETDATASOURCE_API bool GetItemAttribute(FContentBrowserModule* InContentBrowserModule, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetFolderItemAttribute(const FContentBrowserAssetFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetFileItemAttribute(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERASSETDATASOURCE_API bool GetUnsupportedAssetFileItemAttribute(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);                       

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetDataAttribute(const FAssetData& InAssetData, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERASSETDATASOURCE_API bool GetItemAttributes(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetFileItemAttributes(const FContentBrowserAssetFileItemDataPayload& InAssetPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERASSETDATASOURCE_API bool GetUnsupportedAssetFileItemAttributes(const FContentBrowserUnsupportedAssetFileItemDataPayload& InUnsupportedAssetPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetDataAttributes(const FAssetData& InAssetData, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERASSETDATASOURCE_API bool GetAssetAccessSpecifier(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier);

	CONTENTBROWSERASSETDATASOURCE_API bool CanModifyAssetAccessSpecifier(IAssetTools* InAssetTools, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERASSETDATASOURCE_API void PopulateAssetFolderContextMenu(UContentBrowserDataSource* InOwnerDataSource, UToolMenu* InMenu, FAssetFolderContextMenu& InAssetFolderContextMenu);

	CONTENTBROWSERASSETDATASOURCE_API void PopulateAssetFileContextMenu(UContentBrowserDataSource* InOwnerDataSource, UToolMenu* InMenu, FAssetFileContextMenu& InAssetFileContextMenu);

}
