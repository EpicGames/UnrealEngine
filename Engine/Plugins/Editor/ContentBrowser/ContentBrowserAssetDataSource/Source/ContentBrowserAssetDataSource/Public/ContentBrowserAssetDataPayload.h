// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/GCObject.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

class UAssetDefinition;
class IAssetTypeActions;
class FAssetThumbnail;
class UFactory;

class FContentBrowserAssetFolderItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	explicit FContentBrowserAssetFolderItemDataPayload(const FName InInternalPath)
		: InternalPath(InInternalPath)
	{
	}

	FName GetInternalPath() const
	{
		return InternalPath;
	}

	UE_API const FString& GetFilename() const;

private:
	FName InternalPath;

	mutable bool bHasCachedFilename = false;
	mutable FString CachedFilename;
};

class FContentBrowserAssetFileItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	UE_API explicit FContentBrowserAssetFileItemDataPayload(FAssetData&& InAssetData);
	UE_API explicit FContentBrowserAssetFileItemDataPayload(const FAssetData& InAssetData);

	const FAssetData& GetAssetData() const
	{
		return AssetData;
	}

	UE_API UPackage* GetPackage(const bool bTryRecacheIfNull = false) const;

	// LoadTags(optional) allows passing specific tags to the linker when loading the asset package (@see ULevel::LoadAllExternalObjectsTag for an example usage)
	UE_API UPackage* LoadPackage(TSet<FName> LoadTags = {}) const;

	UE_API UObject* GetAsset(const bool bTryRecacheIfNull = false) const;

	//  LoadTags (optional) allows passing specific tags to the linker when loading the asset (@see ULevel::LoadAllExternalObjectsTag for an example usage)
	UE_API UObject* LoadAsset(TSet<FName> LoadTags = {}) const;

	UE_API TSharedPtr<IAssetTypeActions> GetAssetTypeActions() const;
	
	UE_API const UAssetDefinition* GetAssetDefinition() const;

	UE_API const FString& GetFilename() const;

	UE_API void UpdateThumbnail(FAssetThumbnail& InThumbnail) const;

private:
	FAssetData AssetData;

	mutable bool bHasCachedPackagePtr = false;
	mutable TWeakObjectPtr<UPackage> CachedPackagePtr;

	mutable bool bHasCachedAssetPtr = false;
	mutable TWeakObjectPtr<UObject> CachedAssetPtr;

	mutable bool bHasCachedAssetTypeActionsPtr = false;
	mutable TWeakPtr<IAssetTypeActions> CachedAssetTypeActionsPtr;

	mutable bool bHasCachedAssetDefinitionPtr = false;
	mutable TWeakObjectPtr<const UAssetDefinition> CachedAssetDefinitionPtr;

	mutable bool bHasCachedFilename = false;
	mutable FString CachedFilename;

	UE_API void FlushCaches() const;
};

class FContentBrowserAssetFileItemDataPayload_Creation : public FContentBrowserAssetFileItemDataPayload, public FGCObject
{
public:
	UE_API FContentBrowserAssetFileItemDataPayload_Creation(FAssetData&& InAssetData, UClass* InAssetClass, UFactory* InFactory);

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(AssetClass);
		Collector.AddReferencedObject(Factory);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FContentBrowserAssetFileItemDataPayload_Creation");
	}

	UClass* GetAssetClass() const
	{
		return AssetClass;
	}

	UFactory* GetFactory() const
	{
		return Factory;
	}

private:
	/** The class to use when creating the asset */
	TObjectPtr<UClass> AssetClass = nullptr;

	/** The factory to use when creating the asset. */
	TObjectPtr<UFactory> Factory = nullptr;
};

class FContentBrowserAssetFileItemDataPayload_Duplication : public FContentBrowserAssetFileItemDataPayload
{
public:
	UE_API FContentBrowserAssetFileItemDataPayload_Duplication(FAssetData&& InAssetData, TWeakObjectPtr<UObject> InSourceObject);

	UObject* GetSourceObject() const
	{
		return SourceObject.Get();
	}

private:
	/** The context to use when creating the asset. Used when initializing an asset with another related asset. */
	TWeakObjectPtr<UObject> SourceObject;
};

class FContentBrowserUnsupportedAssetFileItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	// Unsupported asset file but it does have an asset data
	UE_API explicit FContentBrowserUnsupportedAssetFileItemDataPayload(FAssetData&& InAssetData);
	UE_API explicit FContentBrowserUnsupportedAssetFileItemDataPayload(const FAssetData& InAssetData);

	UE_API const FAssetData* GetAssetDataIfAvailable() const;

	UE_API const FString& GetFilename() const;

	UE_API UPackage* GetPackage() const;

private:
	UE_API void FlushCaches() const;


	TUniquePtr<FAssetData> OptionalAssetData;

	mutable bool bHasCachedPackagePtr = false;
	mutable TWeakObjectPtr<UPackage> CachedPackagePtr;

	mutable bool bHasCachedFilename = false;
	mutable FString CachedFilename;
};

#undef UE_API
