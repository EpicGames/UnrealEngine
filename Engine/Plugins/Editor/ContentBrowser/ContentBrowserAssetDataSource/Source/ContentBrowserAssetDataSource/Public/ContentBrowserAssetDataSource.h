// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"
#include "ContentBrowserDataSource.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ContentBrowserDataFilter.h"
#include "Misc/NamePermissionList.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserAssetDataSource.generated.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

class FContentBrowserAssetFileItemDataPayload;
class FContentBrowserAssetFolderItemDataPayload;
class FContentBrowserUnsupportedAssetFileItemDataPayload;
class FReply;
struct FPropertyChangedEvent;

class IAssetTools;
class IAssetTypeActions;
class ICollectionManager;
class UFactory;
class UToolMenu;
class FAssetFolderContextMenu;
class FAssetFileContextMenu;
struct FCollectionNameType;
class FContentBrowserModule;
class UContentBrowserAssetDataSource;
class UContentBrowserToolbarMenuContext;

USTRUCT()
struct FContentBrowserCompiledAssetDataFilter
{
	GENERATED_BODY()

public:
	// Folder filtering
	bool bRunFolderQueryOnDemand = false;
	// On-demand filtering (always recursive on PathToScanOnDemand)
	bool bRecursivePackagePathsToInclude = false;
	bool bRecursivePackagePathsToExclude = false;
	FPathPermissionList PackagePathsToInclude;
	FPathPermissionList PackagePathsToExclude;
	FPathPermissionList PathPermissionList;
	TSet<FName> ExcludedPackagePaths;
	EContentBrowserItemAttributeFilter ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeAll;
	EContentBrowserItemCategoryFilter ItemCategoryFilter = EContentBrowserItemCategoryFilter::IncludeAll;
	FString VirtualPathToScanOnDemand;
	// Cached filtering
	TSet<FName> CachedSubPaths;

	// Asset filtering
	bool bFilterExcludesAllAssets = false;
	FARCompiledFilter InclusiveFilter;
	FARCompiledFilter ExclusiveFilter;

	// Legacy custom assets
	TArray<FAssetData> CustomSourceAssets;
};

USTRUCT()
struct FContentBrowserCompiledUnsupportedAssetDataFilter
{
	GENERATED_BODY()

public:
	// Filter that are used to determine if an asset is supported
	FARCompiledFilter ConvertIfFailInclusiveFilter;
	FARCompiledFilter ConvertIfFailExclusiveFilter;

	// Filter for where to show or not the unsupported assets
	FARCompiledFilter ShowInclusiveFilter;
	FARCompiledFilter ShowExclusiveFilter;

	// Standard asset filter modified for the need of the unsupported assets
	FARCompiledFilter InclusiveFilter;
	FARCompiledFilter ExclusiveFilter;
};

enum class EContentBrowserFolderAttributes : uint8;

UCLASS(MinimalAPI)
class UContentBrowserAssetDataSource : public UContentBrowserDataSource
{
	GENERATED_BODY()

public:
	UE_API void Initialize(const bool InAutoRegister = true);

	UE_API virtual void Shutdown() override;

	/**
	 * A struct used to cache some data to accelerate the compilation of the filters for the asset data source
	 */
	struct FAssetDataSourceFilterCache
	{
	public:
		UE_API FAssetDataSourceFilterCache();
		UE_API ~FAssetDataSourceFilterCache();

		FAssetDataSourceFilterCache(const FAssetDataSourceFilterCache&) = delete;
		FAssetDataSourceFilterCache(FAssetDataSourceFilterCache&&) = delete;

		FAssetDataSourceFilterCache& operator=(const FAssetDataSourceFilterCache&) = delete;
		FAssetDataSourceFilterCache& operator=(FAssetDataSourceFilterCache&&) = delete;

		UE_API void RemoveUnusedCachedData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter);
		UE_API void ClearCachedData(const FContentBrowserDataFilterCacheIDOwner& IDOwner);
		UE_API void Reset();

	private:
		friend UContentBrowserAssetDataSource;

		UE_API bool GetCachedCompiledInternalPaths(const FContentBrowserDataFilter& InFilter, FName InVirtualPath, TSet<FName>& OutCompiledInternalPath) const;
		UE_API void CacheCompiledInternalPaths(const FContentBrowserDataFilter& InFilter, FName InVirtualPath, const TSet<FName>& CompiledInternalPaths);

		UE_API void OnPathAdded(FName Path, FStringView PathString, uint32 PathHash, FName ParentPath, uint32 ParentPathHash, int32 PathDepth);
		UE_API void OnPathRemoved(FName Path, uint32 PathHash);

		struct FCachedDataPerID
		{
			TMap<FName, TSet<FName>> InternalPaths;
			EContentBrowserItemAttributeFilter ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeAll;
		};

		/**
		 * A cache for the recursive internal paths (Possible improvement: Rework the cache into a shared cache across multiple IDs for the simple filters and a not shared cache for the more complex filters)
		 */
		TMap<FContentBrowserDataFilterCacheID, FCachedDataPerID> CachedCompiledInternalPaths;
	};

	/**
	 * All of the data necessary to generate a compiled filter for folders and assets
	 */
	struct FAssetFilterInputParams
	{
		TSet<FName> InternalPaths;

		UContentBrowserDataSource* DataSource = nullptr;
		UE_DEPRECATED(5.6, "CollectionManager is no longer used.")
		ICollectionManager* CollectionManager = nullptr;
		FAssetDataSourceFilterCache* AssetFilterCache = nullptr;
		IAssetRegistry* AssetRegistry = nullptr;

		const FContentBrowserDataObjectFilter* ObjectFilter = nullptr;
		const FContentBrowserDataPackageFilter* PackageFilter = nullptr;
		const FContentBrowserDataClassFilter* ClassFilter = nullptr;
		const FContentBrowserDataCollectionFilter* CollectionFilter = nullptr;

		// Filter that convert showed items as unsupported items
		const FContentBrowserDataUnsupportedClassFilter* UnsupportedClassFilter = nullptr;

		const FPathPermissionList* PathPermissionList = nullptr;
		const FPathPermissionList* ClassPermissionList = nullptr;

		FContentBrowserDataFilterList* FilterList = nullptr;
		FContentBrowserCompiledAssetDataFilter* AssetDataFilter = nullptr;
		FContentBrowserCompiledUnsupportedAssetDataFilter* ConvertToUnsupportedAssetDataFilter = nullptr;
		

		bool bIncludeFolders = false;
		bool bIncludeFiles = false;
		bool bIncludeAssets = false;
		bool bIncludeRedirectors = false;
	};

	typedef TFunctionRef<void(const FCollectionRef&, ECollectionRecursionFlags::Flags, TFunctionRef<void(const FSoftObjectPath&)>)> FCollectionEnumerationFunc;
	typedef TFunctionRef<void(FName, TFunctionRef<bool(FName)>, bool)> FSubPathEnumerationFunc;
	typedef TFunctionRef<void(FARFilter&, FARCompiledFilter&)>         FCompileARFilterFunc;
	typedef TFunctionRef<FContentBrowserItemData(FName)>               FCreateFolderItemFunc;

	/**
	 * Call in CompileFilter() to populate an FAssetFilterInputParams for use in CreatePathFilter and CreateAssetFilter.
	 *
	 * @param Params The FAssetFilterInputParams struct to populate with data
	 * @param DataSource The DataSource that CompileFilter() is being called on
	 * @param InAssetRegistry A pointer to the Asset Registry to save time having to find it
	 * @param InFilter The input filter supplied to CompileFilter()
	 * @param OutCompiledFilter The output filer supplied to CompileFilter()
	 * @param CollectionManager If set, this will be used to filter objects when a collection filter is requested. If not set, and a collection filter is requested, the function will return false.
	 * 
	 * @return false if it's not possible to display folders or assets, otherwise true
	 */
	static UE_API bool PopulateAssetFilterInputParams(FAssetFilterInputParams& Params, UContentBrowserDataSource* DataSource, IAssetRegistry* InAssetRegistry, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, ICollectionManager* CollectionManager = nullptr, FAssetDataSourceFilterCache* InFilterCache = nullptr);
	
	/**
	 * Call in CompileFilter() after PopulateAssetFilterInputParams() to fill OutCompiledFilter with an FContentBrowserCompiledAssetDataFilter capable of filtering folders
	 *
	 * @param Params The params generated from PopulateAssetFilterInputParams()
	 * @param InPath The input path supplied to CompileFilter()
	 * @param InFilter The input filter supplied to CompileFilter()
	 * @param OutCompiledFilter The output filer supplied to CompileFilter()
	 * @param SubPathEnumeration A function that calls its input function on all subpaths of the given input path, optionally recursive
	 *
	 * @return false if it's not possible to display any folders, otherwise true
	 */
	static UE_API bool CreatePathFilter(FAssetFilterInputParams& Params, FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, FSubPathEnumerationFunc SubPathEnumeration);
	
	/**
	 * Call in CompileFilter() after CreatePathFilter() to fill OutCompiledFilter with an FContentBrowserCompiledAssetDataFilter capable of filtering assets
	 *
	 * @param Params The params generated from PopulateAssetFilterInputParams()
	 * @param InPath The input path supplied to CompileFilter()
	 * @param InFilter The input filter supplied to CompileFilter()
	 * @param OutCompiledFilter The output filer supplied to CompileFilter()
	 * @param FGetSubPathsFunc A optional function to override how the package paths are expended.
	 *
	 * @return false if it's not possible to display any assets, otherwise true
	 */
	static UE_API bool CreateAssetFilter(FAssetFilterInputParams& Params, FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, FSubPathEnumerationFunc* GetSubPackagePathsFunc = nullptr, FCollectionEnumerationFunc* GetCollectionObjectPathsFunc = nullptr);

	UE_DEPRECATED(5.3, "Use the override without the function to customize the compilation the ARFilter or duplicate this function logic. This function will no longer be supported and will be removed to allow future performence improvements")
	static UE_API bool CreateAssetFilter(FAssetFilterInputParams& Params, FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter, FCompileARFilterFunc CreateCompiledFilter);
	
	/**
	 * Call in EnumerateItemsMatchingFilter() to generate a list of folders that match the compiled filter.
	 * It is the caller's responsibility to verify EContentBrowserItemTypeFilter::IncludeFolders is set before enumerating.
	 *
	 * @param DataSource The DataSource that EnumerateItemsMatchingFilter is being called on
	 * @param AssetDataFilter The filter to use when deciding whether a path is a valid folder
	 * @param InCallback The callback function supplied by EnumerateItemsMatchingFilter()
	 * @param SubPathEnumeration A function that calls its input function on all subpaths of the given input path, optionally recursive
	 * @param CreateFolderItem A function that generates an FContentBrowserItemData folder for the given input path
	 *
	 * @return 
	 */
	static UE_API void EnumerateFoldersMatchingFilter(UContentBrowserDataSource* DataSource, const FContentBrowserCompiledAssetDataFilter* AssetDataFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink, FSubPathEnumerationFunc SubPathEnumeration, FCreateFolderItemFunc CreateFolderItem);
	
/**
	 * Call in DoesItemPassFilter() to check if a folder passes the compiled asset data filter.
	 * It is the caller's responsibility to verify EContentBrowserItemTypeFilter::IncludeFolders is set before enumerating.
	 * 
	 * @param DataSource The DataSource that DoesItemPassFilter() is being called on
	 * @param InItem The folder item to test against the filter supplied by DoesItemPassFilter()
	 * @param Filter The compiled filter to test with supplied by DoesItemPassFilter()
	 *
	 * @return true if the folder passes the filter
	 */
	static UE_API bool DoesItemPassFolderFilter(UContentBrowserDataSource* DataSource, const FContentBrowserItemData& InItem, const FContentBrowserCompiledAssetDataFilter& Filter);

	UE_API virtual void CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter) override;

	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;
	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink) override;

	UE_API virtual void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual bool EnumerateItemsAtPaths(const TArrayView<FContentBrowserItemPath> InPaths, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual bool IsDiscoveringItems(FText* OutStatus = nullptr) override;

	UE_API virtual bool PrioritizeSearchPath(const FName InPath) override;

	UE_API virtual bool IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& InContentsFilter) override;

	UE_API virtual bool CanCreateFolder(const FName InPath, FText* OutErrorMsg) override;

	UE_API virtual bool CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter, FContentBrowserItemDataTemporaryContext& OutPendingItem) override;

	UE_API virtual bool DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter) override;

	UE_API virtual bool ConvertItemForFilter(FContentBrowserItemData& Item, const FContentBrowserDataCompiledFilter& InFilter) override;

	UE_API virtual bool GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue) override;

	UE_API virtual bool GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues) override;

	UE_API virtual bool GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath) override;

	UE_API virtual bool GetItemAssetAccessSpecifier(const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier) override;

	UE_API virtual bool CanModifyItemAssetAccessSpecifier(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool IsItemDirty(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool EditItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkEditItems(TArrayView<const FContentBrowserItemData> InItems) override;
	
	UE_API virtual bool CanViewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool ViewItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkViewItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool PreviewItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanDuplicateItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool DuplicateItem(const FContentBrowserItemData& InItem, FContentBrowserItemDataTemporaryContext& OutPendingItem) override;

	UE_API virtual bool BulkDuplicateItems(TArrayView<const FContentBrowserItemData> InItems, TArray<FContentBrowserItemData>& OutNewItems) override;

	UE_API virtual bool CanSaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg) override;

	UE_API virtual bool SaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags) override;

	UE_API virtual bool BulkSaveItems(TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags) override;

	UE_API virtual bool CanDeleteItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool DeleteItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkDeleteItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanPrivatizeItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	UE_API virtual bool PrivatizeItem(const FContentBrowserItemData& InItem, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private) override;

	UE_API virtual bool BulkPrivatizeItems(TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private) override;

	UE_API virtual bool CanRenameItem(const FContentBrowserItemData& InItem, const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg) override;

	UE_API virtual bool RenameItem(const FContentBrowserItemData& InItem, const FString& InNewName, FContentBrowserItemData& OutNewItem) override;

	UE_API virtual bool CanCopyItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg) override;

	UE_API virtual bool CopyItem(const FContentBrowserItemData& InItem, const FName InDestPath) override;

	UE_API virtual bool BulkCopyItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath) override;

	UE_API virtual bool CanMoveItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg) override;

	UE_API virtual bool MoveItem(const FContentBrowserItemData& InItem, const FName InDestPath) override;

	UE_API virtual bool BulkMoveItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath) override;

	UE_API virtual bool AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr) override;

	UE_API virtual bool AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr) override;

	UE_API virtual bool AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr) override;

	UE_API virtual bool UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail) override;

	UE_API virtual bool HandleDragEnterItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent) override;

	UE_API virtual bool HandleDragOverItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent) override;

	UE_API virtual bool HandleDragLeaveItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent) override;

	UE_API virtual bool HandleDragDropOnItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent) override;

	UE_API virtual bool TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId) override;

	UE_API virtual bool Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath) override;

	UE_API virtual bool Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData) override;

	UE_API virtual bool Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath) override;

	UE_API virtual bool Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath) override;

	static UE_API bool PathPassesCompiledDataFilter(const FContentBrowserCompiledAssetDataFilter& InFilter, const FName InPath);

	UE_API virtual void RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter) override;

	UE_API virtual void ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner) override;

protected:
	UE_API virtual void BuildRootPathVirtualTree() override;

private:
	UE_API bool IsKnownContentPath(const FName InPackagePath) const;

	static UE_API bool GetObjectPathsForCollections(TArrayView<const FCollectionRef> InCollections, const bool bIncludeChildCollections, FCollectionEnumerationFunc* GetCollectionObjectPathsFunc, TArray<FSoftObjectPath>& OutObjectPaths);

	UE_API FContentBrowserItemData CreateAssetFolderItem(const FName InInternalFolderPath);

	UE_API FContentBrowserItemData CreateAssetFileItem(const FAssetData& InAssetData);

	UE_API FContentBrowserItemData CreateUnsupportedAssetFileItem(const FAssetData& InAssetData);

	UE_API TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> GetAssetFolderItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API TSharedPtr<const FContentBrowserAssetFileItemDataPayload> GetAssetFileItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API TSharedPtr<const FContentBrowserUnsupportedAssetFileItemDataPayload> GetUnsupportedAssetFileItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API bool CanHandleDragDropEvent(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent) const;

	UE_API void OnAssetRegistryFileLoadProgress(const IAssetRegistry::FFileLoadProgressUpdateData& InProgressUpdateData);

	UE_API void OnAssetsAdded(TConstArrayView<FAssetData> InAssets);

	UE_API void OnAssetRemoved(const FAssetData& InAssetData);

	UE_API void OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath);

	UE_API void OnAssetUpdated(const FAssetData& InAssetData);

	UE_API void OnAssetUpdatedOnDisk(const FAssetData& InAssetData);

	UE_API void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	UE_API void OnObjectPreSave(UObject* InObject, class FObjectPreSaveContext InObjectPreSaveContext);

	UE_API void OnPathsAdded(TConstArrayView<FStringView> Paths);

	UE_API void OnPathsRemoved(TConstArrayView<FStringView> Paths);

	UE_API void OnPathPopulated(const FAssetData& InAssetData);

	UE_API void OnPathPopulated(const FStringView InPath, const EContentBrowserFolderAttributes InAttributesToSet);

	UE_API void OnAlwaysShowPath(const FString& InPath);

	UE_API void OnContentPathMounted(const FString& InAssetPath, const FString& InFileSystemPath);

	UE_API void OnContentPathDismounted(const FString& InAssetPath, const FString& InFileSystemPath);

	UE_API EContentBrowserFolderAttributes GetAssetFolderAttributes(const FName InPath) const;

	UE_API bool SetAssetFolderAttributes(const FName InPath, const EContentBrowserFolderAttributes InAttributesToSet);

	UE_API bool ClearAssetFolderAttributes(const FName InPath, const EContentBrowserFolderAttributes InAttributesToClear);

	UE_API bool HideFolderIfEmpty(const IContentBrowserHideFolderIfEmptyFilter& HideFolderIfEmptyFilter, FName Path, FStringView PathString) const;

	UE_API void PopulateAddNewContextMenu(UToolMenu* InMenu);

	UE_API void PopulateContentBrowserToolBar(UToolMenu* InMenu);

	UE_API void PopulateAssetFolderContextMenu(UToolMenu* InMenu);

	UE_API void PopulateAssetFileContextMenu(UToolMenu* InMenu);

	UE_API void PopulateDragDropContextMenu(UToolMenu* InMenu);

	UE_API void OnAdvancedCopyRequested(const TArray<FName>& InAdvancedCopyInputs, const FString& InDestinationPath);

	UE_API void OnImportAsset(const FName InPath);

	UE_API void OnNewAssetRequested(const FName InPath, TWeakObjectPtr<UClass> InFactoryClass, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation InOnBeginItemCreation);

	UE_API void OnBeginCreateAsset(const FName InDefaultAssetName, const FName InPackagePath, UClass* InAssetClass, UFactory* InFactory, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation InOnBeginItemCreation);

	UE_API bool OnValidateItemName(const FContentBrowserItemData& InItem, const FString& InProposedName, FText* OutErrorMsg);

	UE_API FReply OnImportClicked(const UContentBrowserToolbarMenuContext* ContextObject);

	UE_API bool IsImportEnabled(const UContentBrowserToolbarMenuContext* ContextObject) const;

	UE_API FContentBrowserItemData OnFinalizeCreateFolder(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg);

	UE_API FContentBrowserItemData OnFinalizeCreateAsset(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg);

	UE_API FContentBrowserItemData OnFinalizeDuplicateAsset(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg);

	UE_API void AddRootContentPathToStateMachine(const FString& InAssetPath);

	UE_API void RemoveRootContentPathFromStateMachine(const FString& InAssetPath);
	
	UE_API void OnExtendContentBrowserCommands(TSharedRef<FUICommandList> InCommandList, FOnContentBrowserGetSelection InGetSelectionDelegate);

	IAssetRegistry* AssetRegistry = nullptr;

	IAssetTools* AssetTools = nullptr;

	FContentBrowserModule* ContentBrowserModule = nullptr;

	ICollectionManager* CollectionManager;

	TSharedPtr<FAssetFolderContextMenu> AssetFolderContextMenu;

	TSharedPtr<FAssetFileContextMenu> AssetFileContextMenu;

	FText DiscoveryStatusText;

	/**
	 * The array of known root content paths that can hold assets.
	 * @note These paths include a trailing slash.
	 */
	TArray<FString> RootContentPaths;

	struct FCharacterNode;

	struct FCharacterNodePtr
	{
		FCharacterNodePtr()
			: Node(MakeUnique<FCharacterNode>())
		{
		}

		FCharacterNodePtr(FCharacterNodePtr&&) = default;
		FCharacterNodePtr& operator=(FCharacterNodePtr&&) = default;

		FCharacterNodePtr(const FCharacterNodePtr&) = delete;
		FCharacterNodePtr& operator=(const FCharacterNodePtr&) = delete;

		const FCharacterNode& operator*() const
		{
			return Node.operator*();
		}

		const FCharacterNode* operator->() const
		{
			return Node.operator->();
		}

		FCharacterNode& operator*()
		{
			return Node.operator*();
		}

		FCharacterNode* operator->()
		{
			return Node.operator->();
		}
 
		const FCharacterNode* Get() const
		{
			return Node.Get();
		}

		FCharacterNode* Get()
		{
			return Node.Get();
		}

	private:

		TUniquePtr<FCharacterNode> Node;
	};

	struct FCharacterNode
	{
		// The next characters in the tree and the number of paths beginning with the prefix including that character for use in removing paths.
		TMap<TCHAR, TPair<FCharacterNodePtr, int32>> NextNodes;

		bool bIsEndOfAMountPoint = false;
	};

	// Tree of character nodes all in lower case. Used to speed up queries against the RootContentPaths Array.
	FCharacterNode RootContentPathsTrie;

	/**
	 * Map of folders that have attributes set.
	 */
	TMap<FName, EContentBrowserFolderAttributes> AssetFolderToAttributes;

	/**
	 * A cache of folders that have been populated since the last time any new asset folders were added.
	 */
	TMap<FName, EContentBrowserFolderAttributes> RecentlyPopulatedAssetFolders;

	DECLARE_MULTICAST_DELEGATE_SixParams(FOnAssetDataSourcePathAdded, FName, FStringView, uint32, FName, uint32, int32);
	static UE_API FOnAssetDataSourcePathAdded OnAssetPathAddedDelegate;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetDataSourcePathRemoved, FName, uint32);
	static UE_API FOnAssetDataSourcePathRemoved OnAssetPathRemovedDelegate;

	FAssetDataSourceFilterCache FilterCache;
};

#undef UE_API
