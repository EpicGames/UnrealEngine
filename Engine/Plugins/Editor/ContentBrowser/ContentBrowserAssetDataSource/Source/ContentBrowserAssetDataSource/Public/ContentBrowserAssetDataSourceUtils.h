// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "AssetRegistry/AssetData.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

namespace UE::Editor::ContentBrowserAssetDataSource
{
	UE_API int32 CaptureThumbnail(const TArray<FAssetData>& InAssets);
	UE_API bool CanCaptureThumbnail(const TArray<FAssetData>& InAssets);
}

#undef UE_API
