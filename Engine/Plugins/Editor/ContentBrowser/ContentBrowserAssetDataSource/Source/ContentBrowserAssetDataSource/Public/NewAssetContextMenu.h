// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeCategories.h"
#include "Framework/Commands/UIAction.h"

#define UE_API CONTENTBROWSERASSETDATASOURCE_API

class UToolMenu;
struct FCategorySubMenuItem;

class FNewAssetContextMenu
{
public:
	DECLARE_DELEGATE_OneParam(FOnImportAssetRequested, const FName /*SelectedPath*/);
	DECLARE_DELEGATE_TwoParams(FOnNewAssetRequested, const FName /*SelectedPath*/, TWeakObjectPtr<UClass> /*FactoryClass*/);

	/** Makes the context menu widget */
	static UE_API void MakeContextMenu(
		UToolMenu* Menu,
		const TArray<FName>& InSelectedAssetPaths,
		const FOnImportAssetRequested& InOnImportAssetRequested,
		const FOnNewAssetRequested& InOnNewAssetRequested
		);

private:
	/** Handle creating a new asset from an asset category */
	static UE_API void CreateNewAssetMenuCategory(UToolMenu* Menu, FName SectionName, EAssetTypeCategories::Type AssetTypeCategory, FName InPath, FOnNewAssetRequested InOnNewAssetRequested, FCanExecuteAction InCanExecuteAction);
	static UE_API void CreateNewAssetMenus(UToolMenu* Menu, FName SectionName, TSharedPtr<FCategorySubMenuItem> SubMenuData, FName InPath, FOnNewAssetRequested InOnNewAssetRequested, FCanExecuteAction InCanExecuteAction);

	/** Handle when the "Import" button is clicked */
	static UE_API void ExecuteImportAsset(FOnImportAssetRequested InOnImportAssetRequested, FName InPath);

	/** Create a new asset using the specified factory at the specified path */
	static UE_API void ExecuteNewAsset(FOnNewAssetRequested InOnNewAssetRequested, FName InPath, TWeakObjectPtr<UClass> FactoryClass);
};

#undef UE_API
